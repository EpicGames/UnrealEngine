// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneProxy.h"
#include "IPersonaPreviewScene.h"
#include "BoneControllers/AnimNode_ModifyBone.h"
#include "AnimPreviewInstance.h"
#include "Editor.h"
#include "IPersonaToolkit.h"
#include "SAdvancedTransformInputBox.h"
#include "ScopedTransaction.h"
#include "Animation/DebugSkelMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneProxy)

#define LOCTEXT_NAMESPACE "BoneProxy"

UBoneProxy::UBoneProxy()
	: bLocalLocation(true)
	, bLocalRotation(true)
	, bLocalScale(true)
	, PreviousLocation(FVector::ZeroVector)
	, PreviousRotation(FRotator::ZeroRotator)
	, PreviousScale(FVector::ZeroVector)
	, bManipulating(false)
	, bIsTickable(false)
	, bIsTransformEditable(true)
{
}

FTransform GetWorldSpaceBoneTransform(const FReferenceSkeleton& ReferenceSkeleton, const int32 BoneIndex)
{
	const TArray<FTransform>& BonePoses = ReferenceSkeleton.GetRefBonePose();

	if (BonePoses.IsValidIndex(BoneIndex))
	{
		FTransform WorldSpacePose = BonePoses[BoneIndex];

		int32 ParentIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);

		while(ParentIndex != INDEX_NONE)
		{
			WorldSpacePose = WorldSpacePose * BonePoses[ParentIndex];
			ParentIndex = ReferenceSkeleton.GetParentIndex(ParentIndex);
		}

		return WorldSpacePose;		
	}

	return FTransform::Identity;
}

void UBoneProxy::Tick(float DeltaTime)
{
	if (!bManipulating)
	{
		if (UDebugSkelMeshComponent* Component = SkelMeshComponent.Get())
		{
			if (Component->GetSkeletalMeshAsset() && Component->GetSkeletalMeshAsset()->IsCompiling())
			{
				//We do not want to tick if the skeletal mesh is inside a compilation
				return;
			}

			if (Component->GetSkeletalMeshAsset())
			{
				TArray<FTransform> LocalBoneTransforms = Component->GetBoneSpaceTransforms();

				const int32 BoneIndex = Component->GetBoneIndex(BoneName);
				if (LocalBoneTransforms.IsValidIndex(BoneIndex))
				{
					const FTransform& LocalTransform = LocalBoneTransforms[BoneIndex];
					const FTransform BoneTransform = Component->GetBoneTransform(BoneIndex);

					if (bLocalLocation)
					{
						Location = LocalTransform.GetLocation();
					}
					else
					{
						Location = BoneTransform.GetLocation();
					}

					if (bLocalRotation)
					{
						Rotation = LocalTransform.GetRotation().Rotator();
					}
					else
					{
						Rotation = BoneTransform.GetRotation().Rotator();
					}

					if(bLocalScale)
					{
						Scale = LocalTransform.GetScale3D();
					}
					else
					{
						Scale = BoneTransform.GetScale3D();
					}

					const FTransform ReferenceTransform = Component->GetSkeletalMeshAsset()->GetRefSkeleton().GetRefBonePose()[BoneIndex];
					ReferenceLocation = ReferenceTransform.GetLocation();
					ReferenceRotation = ReferenceTransform.GetRotation().Rotator();
					ReferenceScale = ReferenceTransform.GetScale3D();
				}

				// Show mesh relative transform on the details panel so we have a way to visualize the root transform when processing root motion
				// Note that this doesn't always represent the actual transform of the root in the animation at current time but where root motion has taken us so far
				// It will not match the root transform at the current time in the animation after lopping multiple times if we are using ProcessRootMotion::Loop
				// or if we are visualizing a complex section from a montage, for example
				const FTransform MeshRelativeTransform = Component->GetRelativeTransform();
				MeshLocation = MeshRelativeTransform.GetLocation();
				MeshRotation = MeshRelativeTransform.GetRotation().Rotator();
				MeshScale = MeshRelativeTransform.GetScale3D();
			}
			else if (const TSharedPtr<IPersonaPreviewScene> PreviewScene = WeakPreviewScene.Pin())
			{
				if (USkeleton* Skeleton = PreviewScene->GetPersonaToolkit()->GetSkeleton())
				{
					const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
					if (BoneIndex != INDEX_NONE)
					{
						const FTransform& ReferenceTransform = Skeleton->GetReferenceSkeleton().GetRefBonePose()[BoneIndex];
						const FTransform WorldReferenceTransform = GetWorldSpaceBoneTransform(Skeleton->GetReferenceSkeleton(), BoneIndex);
					
						if (bLocalLocation)
						{
							Location = ReferenceTransform.GetLocation();
						}
						else
						{
							Location = WorldReferenceTransform.GetLocation();
						}

						if (bLocalRotation)
						{
							Rotation = ReferenceTransform.GetRotation().Rotator();
						}
						else
						{
							Rotation = WorldReferenceTransform.GetRotation().Rotator();
						}

						if(bLocalScale)
						{
							Scale = ReferenceTransform.GetScale3D();
						}
						else
						{
							Scale = WorldReferenceTransform.GetScale3D();
						}

						ReferenceLocation = ReferenceTransform.GetLocation();
						ReferenceRotation = ReferenceTransform.GetRotation().Rotator();
						ReferenceScale = ReferenceTransform.GetScale3D();
					}
				}
			}
		}
	}
}

bool UBoneProxy::IsTickable() const
{
	return bIsTickable;
}

TStatId UBoneProxy::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UBoneProxy, STATGROUP_Tickables);
}

TOptional<FVector::FReal> UBoneProxy::GetNumericValue(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	ETransformType TransformType) const
{
	const FVector* LocationPtr = &Location;
	const FRotator* RotationPtr = &Rotation;
	const FVector* ScalePtr = &Scale;

	switch(TransformType)
	{
		case TransformType_Bone:
			// Pointers are already set.
			break;
		case TransformType_Reference:
		{
			LocationPtr = &ReferenceLocation;
			RotationPtr = &ReferenceRotation;
			ScalePtr = &ReferenceScale;
			break;
		}
		case TransformType_Mesh:
		{
			LocationPtr = &MeshLocation;
			RotationPtr = &MeshRotation;
			ScalePtr = &MeshScale;
			break;
		}
	}

	const FEulerTransform Transform(*RotationPtr, *LocationPtr, *ScalePtr);
	return SAdvancedTransformInputBox<FEulerTransform>::GetNumericValueFromTransform(
		Transform,
		Component,
		Representation,
		SubComponent
		);
}

TOptional<FVector::FReal> UBoneProxy::GetMultiNumericValue(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	ETransformType TransformType,
	TArrayView<UBoneProxy*> BoneProxies)
{
	TOptional<FVector::FReal> FirstValue = BoneProxies[0]->GetNumericValue(Component, Representation, SubComponent, TransformType);
	if(!FirstValue.IsSet())
	{
		return FirstValue;
	}

	for(int32 Index=1;Index<BoneProxies.Num();Index++)
	{
		TOptional<FVector::FReal> Value = BoneProxies[Index]->GetNumericValue(Component, Representation, SubComponent, TransformType);
		if(!Value.IsSet())
		{
			return Value;
		}
		if(!FMath::IsNearlyEqual(FirstValue.GetValue(), Value.GetValue()))
		{
			return TOptional<FVector::FReal>();
		}
	}

	return FirstValue;
}

void UBoneProxy::OnNumericValueChanged(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value,
	ETransformType TransformType)
{
	if(TransformType != TransformType_Bone || !bIsTransformEditable)
	{
		return;
	}

	switch(Component)
	{
		case ESlateTransformComponent::Location:
		{
			OnPreEditChange(GET_MEMBER_NAME_CHECKED(UBoneProxy, Location));
				
			switch(SubComponent)
			{
				case ESlateTransformSubComponent::X:
				{
					Location.X = Value;
					break;
				}
				case ESlateTransformSubComponent::Y:
				{
					Location.Y = Value;
					break;
				}
				case ESlateTransformSubComponent::Z:
				{
					Location.Z = Value;
					break;
				}
				default:
					checkNoEntry();
			}

			OnPostEditChangeProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, Location));
			break;
		}
		case ESlateTransformComponent::Rotation:
		{
			OnPreEditChange(GET_MEMBER_NAME_CHECKED(UBoneProxy, Rotation));
				
			switch(SubComponent)
			{
				case ESlateTransformSubComponent::Roll:
				{
					Rotation.Roll = Value;
					break;
				}
				case ESlateTransformSubComponent::Pitch:
				{
					Rotation.Pitch = Value;
					break;
				}
				case ESlateTransformSubComponent::Yaw:
				{
					Rotation.Yaw = Value;
					break;
				}
				default:
					checkNoEntry();
			}

			OnPostEditChangeProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, Rotation));
			break;
		}
		case ESlateTransformComponent::Scale:
		{
			OnPreEditChange(GET_MEMBER_NAME_CHECKED(UBoneProxy, Scale));

			switch(SubComponent)
			{
				case ESlateTransformSubComponent::X:
				{
					Scale.X = Value;
					break;
				}
				case ESlateTransformSubComponent::Y:
				{
					Scale.Y = Value;
					break;
				}
				case ESlateTransformSubComponent::Z:
				{
					Scale.Z = Value;
					break;
				}
				default:
					checkNoEntry();
			}

			OnPostEditChangeProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, Scale));
			break;
		}
		default:
			checkNoEntry();
	}
}


void UBoneProxy::OnSliderMovementStateChanged(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value, 
	ESliderMovementState SliderMovementState,
	TArrayView<UBoneProxy*> BoneProxies
	)
{
	// If doing slider movement, we have to start the transaction at slider move begin so that
	// we can make a copy of the state before value changes begin. The slider end transition
	// occurs after value commit has happened, so that's where we end the transaction.
	if (SliderMovementState == ESliderMovementState::Begin)
	{
		BeginSetValueTransaction(Component, BoneProxies);
	}
	else // == ESliderMovementState::End
	{
		EndTransaction();
	}
	
	for(UBoneProxy* BoneProxy : BoneProxies)
	{
		BoneProxy->bManipulating = SliderMovementState == ESliderMovementState::Begin;
	}
}


void UBoneProxy::OnMultiNumericValueChanged(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value,
	ETextCommit::Type InCommitType,
	bool bInTransactional,
	ETransformType TransformType,
	TArrayView<UBoneProxy*> BoneProxies)
{
	if(TransformType != TransformType_Bone)
	{
		return;
	}

	// Are we in a slider movement? In that case, transaction is handled by the OnSliderMovementStateChanged
	// callback, since that completely brackets calls to OnMultiNumericValueChanged for change/commit values.
	bool bInSliderMovement = false;
	for(UBoneProxy* BoneProxy : BoneProxies)
	{
		if (BoneProxy->bManipulating)
		{
			bInSliderMovement = true;
			break;
		}
	}

	if (bInTransactional && !bInSliderMovement)
	{
		BeginSetValueTransaction(Component, BoneProxies);
	}
	
	for(UBoneProxy* BoneProxy : BoneProxies)
	{
		BoneProxy->OnNumericValueChanged(Component, Representation, SubComponent, Value, TransformType);
	}
	
	if (bInTransactional && !bInSliderMovement)
	{
		EndTransaction();
	}
}

bool UBoneProxy::DiffersFromDefault(ESlateTransformComponent::Type Component, ETransformType TransformType) const
{
	if(TransformType == TransformType_Bone && bIsTransformEditable)
	{
		switch(Component)
		{
			case ESlateTransformComponent::Location:
			{
				return !Location.Equals(ReferenceLocation);
			}
			case ESlateTransformComponent::Rotation:
			{
				return !Rotation.Equals(ReferenceRotation);
			}
			case ESlateTransformComponent::Scale:
			{
				return !Scale.Equals(ReferenceScale);
			}
			default:
			{
				return DiffersFromDefault(ESlateTransformComponent::Location, TransformType) ||
					DiffersFromDefault(ESlateTransformComponent::Rotation, TransformType) ||
						DiffersFromDefault(ESlateTransformComponent::Scale, TransformType);
			}
		}
	}
	return false;
}


void UBoneProxy::ResetToDefault(ESlateTransformComponent::Type InComponent, ETransformType TransformType)
{
	if(TransformType == TransformType_Bone && bIsTransformEditable)
	{
		if (const UDebugSkelMeshComponent* Component = SkelMeshComponent.Get())
		{
			if (Component->PreviewInstance && Component->AnimScriptInstance == Component->PreviewInstance)
			{
				const int32 BoneIndex = Component->GetBoneIndex(BoneName);
				if (BoneIndex != INDEX_NONE && BoneIndex < Component->GetNumComponentSpaceTransforms())
				{
					Component->PreviewInstance->SetFlags(RF_Transactional);
					Component->PreviewInstance->Modify();

					FAnimNode_ModifyBone& ModifyBone = Component->PreviewInstance->ModifyBone(BoneName);

					switch(InComponent)
					{
						case ESlateTransformComponent::Location:
						{
							ModifyBone.Translation = FVector::ZeroVector;
							break;
						}
						case ESlateTransformComponent::Rotation:
						{
							ModifyBone.Rotation = FRotator::ZeroRotator;
							break;
						}
						case ESlateTransformComponent::Scale:
						{
							ModifyBone.Scale = FVector::OneVector;
							break;
						}
						default:
						{
							ModifyBone.Translation = FVector::ZeroVector;
							ModifyBone.Rotation = FRotator::ZeroRotator;
							ModifyBone.Scale = FVector::OneVector;
							break;
						}
					}

					if(ModifyBone.Translation.Equals(FVector::ZeroVector) &&
						ModifyBone.Rotation.Equals(FRotator::ZeroRotator) &&
						ModifyBone.Scale.Equals(FVector::OneVector))
					{
						Component->PreviewInstance->RemoveBoneModification(BoneName);
					}
				}
			}
		}
	}
}

void UBoneProxy::BeginSetValueTransaction(ESlateTransformComponent::Type InComponent, TArrayView<UBoneProxy*> BoneProxies)
{
	FText TransactionScopeText;
	switch (InComponent)
	{
	case ESlateTransformComponent::Location:
		TransactionScopeText = LOCTEXT("SetLocation", "Set Location");
		break;
	case ESlateTransformComponent::Rotation:
		TransactionScopeText = LOCTEXT("SetRotation", "Set Rotation");
		break;
	case ESlateTransformComponent::Scale:
		TransactionScopeText = LOCTEXT("SetScale", "Set Scale");
		break;
	default:
		checkNoEntry();
	}

	GEditor->BeginTransaction(TransactionScopeText);

	for (UBoneProxy* BoneProxy: BoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			if (Component->PreviewInstance && Component->AnimScriptInstance == Component->PreviewInstance)
			{
				Component->PreviewInstance->SetFlags(RF_Transactional);
				Component->PreviewInstance->Modify();
			}
		}
	}
}

void UBoneProxy::EndTransaction()
{
	GEditor->EndTransaction();
}

void UBoneProxy::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	if (UDebugSkelMeshComponent* Component = SkelMeshComponent.Get())
	{
		if (Component->PreviewInstance && Component->AnimScriptInstance == Component->PreviewInstance)
		{
			if (PropertyAboutToChange.GetActiveMemberNode()->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UBoneProxy, Location))
			{
				PreviousLocation = Location;
			}
			else if (PropertyAboutToChange.GetActiveMemberNode()->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UBoneProxy, Rotation))
			{
				PreviousRotation = Rotation;
			}
			else if (PropertyAboutToChange.GetActiveMemberNode()->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UBoneProxy, Scale))
			{
				PreviousScale = Scale;
			}
		}
	}
}

void UBoneProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if(PropertyChangedEvent.Property != nullptr)
	{
		if (UDebugSkelMeshComponent* Component = SkelMeshComponent.Get())
		{
			if (Component->PreviewInstance && Component->AnimScriptInstance == Component->PreviewInstance)
			{
				int32 BoneIndex = Component->GetBoneIndex(BoneName);
				if (BoneIndex != INDEX_NONE && BoneIndex < Component->GetNumComponentSpaceTransforms())
				{
					FTransform BoneTransform = Component->GetBoneTransform(BoneIndex);
					FMatrix BoneLocalCoordSystem = Component->GetBoneTransform(BoneIndex).ToMatrixNoScale().RemoveTranslation();
					FAnimNode_ModifyBone& ModifyBone = Component->PreviewInstance->ModifyBone(BoneName);
					FTransform ModifyBoneTransform(ModifyBone.Rotation, ModifyBone.Translation, ModifyBone.Scale);
					FTransform BaseTransform = BoneTransform.GetRelativeTransformReverse(ModifyBoneTransform);

					if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBoneProxy, Location))
					{
						FVector Delta = (Location - PreviousLocation);
						if (!Delta.IsNearlyZero())
						{
							if (bLocalLocation)
							{
								Delta = BoneLocalCoordSystem.TransformPosition(Delta);
							}

							FVector BoneSpaceDelta = BaseTransform.TransformVector(Delta);
							ModifyBone.Translation += BoneSpaceDelta;
						}
					}
					else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBoneProxy, Rotation))
					{
						FRotator Delta = (Rotation - PreviousRotation);
						if (!Delta.IsNearlyZero())
						{					
							if (bLocalRotation)
							{
								// get delta in current coord space
								Delta = (BoneLocalCoordSystem.Inverse() * FRotationMatrix(Delta) * BoneLocalCoordSystem).Rotator();
							}

							FVector RotAxis;
							float RotAngle;
							Delta.Quaternion().ToAxisAndAngle(RotAxis, RotAngle);

							FVector4 BoneSpaceAxis = BaseTransform.TransformVectorNoScale(RotAxis);

							//Calculate the new delta rotation
							FQuat NewDeltaQuat(BoneSpaceAxis, RotAngle);
							NewDeltaQuat.Normalize();

							FRotator NewRotation = (ModifyBoneTransform * FTransform(NewDeltaQuat)).Rotator();
							ModifyBone.Rotation = NewRotation;
						}
					}
					else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBoneProxy, Scale))
					{
						FVector Delta = (Scale - PreviousScale);
						if (!Delta.IsNearlyZero())
						{
							ModifyBone.Scale += Delta;
						}
					}
				}
			}
		}
	}
}

void UBoneProxy::OnPreEditChange(FName PropertyName)
{
	FEditPropertyChain PropertyChain;
	FProperty* Property = FindFProperty<FProperty>(GetClass(), PropertyName);
	PropertyChain.AddHead(Property);
	PropertyChain.SetActiveMemberPropertyNode(Property);
	PreEditChange(PropertyChain);
}

void UBoneProxy::OnPostEditChangeProperty(FName PropertyName)
{
	FProperty* Property = FindFProperty<FProperty>(GetClass(), PropertyName);
	FPropertyChangedEvent ChangedEvent(Property, bManipulating ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
	PostEditChangeProperty(ChangedEvent);
}

#undef LOCTEXT_NAMESPACE
