// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneProxyDetailsCustomization.h"
#include "BoneProxy.h"
#include "AnimPreviewInstance.h"
#include "PropertyHandle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBox.h"
#include "Algo/Transform.h"
#include "SAdvancedTransformInputBox.h"

#if WITH_EDITOR

#include "HAL/PlatformApplicationMisc.h"

#endif

#define LOCTEXT_NAMESPACE "FBoneProxyDetailsCustomization"

namespace BoneProxyCustomizationConstants
{
	static const float ItemWidth = 125.0f;
}

static FText GetTransformFieldText(bool* bValuePtr, FText Label)
{
	return *bValuePtr ? FText::Format(LOCTEXT("Local", "Local {0}"), Label) : FText::Format(LOCTEXT("World", "World {0}"), Label);
}

static void OnSetRelativeTransform(bool* bValuePtr)
{
	*bValuePtr = true;
}

static void OnSetWorldTransform(bool* bValuePtr)
{
	*bValuePtr = false;
}

static bool IsRelativeTransformChecked(bool* bValuePtr)
{
	return *bValuePtr;
}

static bool IsWorldTransformChecked(bool* bValuePtr)
{
	return !*bValuePtr;
}

static TSharedRef<SWidget> BuildTransformFieldLabel(bool* bValuePtr, const FText& Label, bool bMultiSelected)
{
	if (bMultiSelected)
	{
		return SNew(STextBlock)
			.Text(Label)
			.Font(IDetailLayoutBuilder::GetDetailFont());
	}
	else
	{
		FMenuBuilder MenuBuilder(true, nullptr, nullptr);

		FUIAction SetRelativeLocationAction
		(
			FExecuteAction::CreateStatic(&OnSetRelativeTransform, bValuePtr),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&IsRelativeTransformChecked, bValuePtr)
		);

		FUIAction SetWorldLocationAction
		(
			FExecuteAction::CreateStatic(&OnSetWorldTransform, bValuePtr),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&IsWorldTransformChecked, bValuePtr)
		);

		MenuBuilder.BeginSection( TEXT("TransformType"), FText::Format( LOCTEXT("TransformType", "{0} Type"), Label ) );

		MenuBuilder.AddMenuEntry
		(
			FText::Format( LOCTEXT( "LocalLabel", "Local"), Label ),
			FText::Format( LOCTEXT( "LocalLabel_ToolTip", "{0} is relative to its parent"), Label ),
			FSlateIcon(),
			SetRelativeLocationAction,
			NAME_None, 
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			FText::Format( LOCTEXT( "WorldLabel", "World"), Label ),
			FText::Format( LOCTEXT( "WorldLabel_ToolTip", "{0} is relative to the world"), Label ),
			FSlateIcon(),
			SetWorldLocationAction,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.EndSection();

		return 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SComboButton)
				.ContentPadding( 0.f )
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ForegroundColor( FSlateColor::UseForeground() )
				.MenuContent()
				[
					MenuBuilder.MakeWidget()
				]
				.ButtonContent()
				[
					SNew( SBox )
					.Padding( FMargin( 0.0f, 0.0f, 2.0f, 0.0f ) )
					[
						SNew(STextBlock)
						.Text_Static(&GetTransformFieldText, bValuePtr, Label)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			];
	}
}

namespace
{

class FBoneProxyDetailsErrorPipe : public FOutputDevice
{
public:

	int32 NumErrors;

	FBoneProxyDetailsErrorPipe()
		: FOutputDevice()
		, NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		NumErrors++;
	}
};

template<typename DataType>
bool GetDataFromContent(const FString& Content, DataType& OutData)
{
	FBoneProxyDetailsErrorPipe ErrorPipe;
	static UScriptStruct* DataStruct = TBaseStructure<DataType>::Get();
	DataStruct->ImportText(*Content, &OutData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);
	return (ErrorPipe.NumErrors == 0);
}
	
}

void FBoneProxyDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	BoneProxies.Empty();
	Algo::TransformIf(Objects, BoneProxies, [](TWeakObjectPtr<UObject> InItem) { return InItem.IsValid() && InItem.Get()->IsA<UBoneProxy>(); }, [](TWeakObjectPtr<UObject> InItem) { return CastChecked<UBoneProxy>(InItem.Get()); });
	TArrayView<UBoneProxy*> BoneProxiesView(BoneProxies);

	UBoneProxy* FirstBoneProxy = CastChecked<UBoneProxy>(Objects[0].Get());

	bool bIsEditingEnabled = true;
	if (UDebugSkelMeshComponent* Component = FirstBoneProxy->SkelMeshComponent.Get())
	{
		bIsEditingEnabled = (Component->AnimScriptInstance == Component->PreviewInstance);
	}

	if (bIsEditingEnabled)
	{
		bool bHasEditable = false;
		for (const UBoneProxy* BoneProxy : BoneProxies)
		{
			if (BoneProxy->bIsTransformEditable)
			{
				bHasEditable = true;
				break;
			}
		}
		bIsEditingEnabled = bHasEditable;
	}
	
	DetailBuilder.HideCategory(TEXT("Transform"));
	DetailBuilder.HideCategory(TEXT("Reference Transform"));
	DetailBuilder.HideCategory(TEXT("Mesh Relative Transform"));
	DetailBuilder.EditCategory(TEXT("Bone")).SetSortOrder(1);

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Transforms"));
	CategoryBuilder.SetSortOrder(2);

	const TArray<FText> ButtonLabels =
	{
		LOCTEXT("BoneTransform", "Bone"),
		LOCTEXT("ReferenceTransform", "Reference"),
		LOCTEXT("MeshTransform", "Mesh Relative")
	};
	const TArray<FText> ButtonTooltips =
	{
		LOCTEXT("BoneTransformTooltip", "The transform of the bone"),
		LOCTEXT("ReferenceTransformTooltip", "The reference transform of a bone (original)"),
		LOCTEXT("MeshTransformTooltip", "The relative transform of the mesh")
	};

	const TArray<UBoneProxy::ETransformType> TransformTypes = {
		UBoneProxy::TransformType_Bone,
		UBoneProxy::TransformType_Reference,
		UBoneProxy::TransformType_Mesh
	};

	static TAttribute<TArray<UBoneProxy::ETransformType>> VisibleTransform = TransformTypes;
	
	TSharedPtr<SSegmentedControl<UBoneProxy::ETransformType>> TransformChoiceWidget =
		SSegmentedControl<UBoneProxy::ETransformType>::Create(
			TransformTypes,
			ButtonLabels,
			ButtonTooltips,
			VisibleTransform
		);

	// use a static shared ref so that all views retain these settings
	static TSharedRef<TArray<UBoneProxy::ETransformType>> VisibleTransforms =
		MakeShareable(new TArray<UBoneProxy::ETransformType>({
			UBoneProxy::TransformType_Bone,
			UBoneProxy::TransformType_Reference,
			UBoneProxy::TransformType_Mesh}));

	CategoryBuilder.AddCustomRow(FText::FromString(TEXT("TransformType")))
	.ValueContent()
	.MinDesiredWidth(375.f)
	.MaxDesiredWidth(375.f)
	.HAlign(HAlign_Left)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			TransformChoiceWidget.ToSharedRef()
		]
	];

	SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FEulerTransform>::FArguments()
	.DisplayRelativeWorld(true)
	.AllowEditRotationRepresentation(false)
	.DisplayScaleLock(true)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.UseQuaternionForRotation(false)
	.OnGetIsComponentRelative_Lambda(
		[BoneProxiesView](ESlateTransformComponent::Type InComponent)
		{
			switch(InComponent)
			{
				case ESlateTransformComponent::Location:
					return BoneProxiesView[0]->bLocalLocation;
				case ESlateTransformComponent::Rotation:
					return BoneProxiesView[0]->bLocalRotation;
				case ESlateTransformComponent::Scale:
					return BoneProxiesView[0]->bLocalScale;
			}
			return true;
		})
	.OnIsComponentRelativeChanged_Lambda(
		[BoneProxiesView](ESlateTransformComponent::Type InComponent, bool bIsRelative)
		{
			for(UBoneProxy* BoneProxy : BoneProxiesView)
			{
				switch(InComponent)
				{
					case ESlateTransformComponent::Location:
					{
						BoneProxy->bLocalLocation = bIsRelative;
						break;
					}
					case ESlateTransformComponent::Rotation:
					{
						BoneProxy->bLocalRotation = bIsRelative;
						break;
					}
					case ESlateTransformComponent::Scale:
					{
						BoneProxy->bLocalScale = bIsRelative;
						break;
					}
				}
			}
		});

	TArray<TSharedRef<IPropertyHandle>> Properties;
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, Location)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, Rotation)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, Scale)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, ReferenceLocation)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, ReferenceRotation)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, ReferenceScale)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, MeshLocation)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, MeshRotation)));
	Properties.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBoneProxy, MeshScale)));

	int32 PropertyIndex = 0;
	for(int32 TransformIndex=0;TransformIndex<3;TransformIndex++)
	{
		// only the first transform can be edited
		const UBoneProxy::ETransformType TransformType = (UBoneProxy::ETransformType)TransformIndex; 
		bIsEditingEnabled = TransformType == UBoneProxy::TransformType_Bone ? bIsEditingEnabled : false;

		TransformWidgetArgs
		.IsEnabled(bIsEditingEnabled)
		.DisplayRelativeWorld(bIsEditingEnabled)
		.DisplayScaleLock(bIsEditingEnabled)
		.OnGetNumericValue_Static(&UBoneProxy::GetMultiNumericValue, TransformType, BoneProxiesView)
		.OnCopyToClipboard_Lambda([TransformType, BoneProxiesView](ESlateTransformComponent::Type InComponent)
		{
			if(BoneProxiesView.Num() == 0)
			{
				return;
			}
			
			if(TransformType == UBoneProxy::TransformType_Bone)
			{
				FString Content;
				UBoneProxy* BoneProxy = BoneProxiesView[0];
				
				switch(InComponent)
				{
					case ESlateTransformComponent::Location:
					{
						const FVector Data = BoneProxy->Location;
						TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
						break;
					}
					case ESlateTransformComponent::Rotation:
					{
						const FRotator Data = BoneProxy->Rotation;
						TBaseStructure<FRotator>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
						break;
					}
					case ESlateTransformComponent::Scale:
					{
						const FVector Data = BoneProxy->Scale;
						TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
						break;
					}
					case ESlateTransformComponent::Max:
					default:
					{
						const FEulerTransform Data(BoneProxy->Location, BoneProxy->Rotation, BoneProxy->Scale);
						TBaseStructure<FEulerTransform>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
						break;
					}
				}

				if(!Content.IsEmpty())
				{
					FPlatformApplicationMisc::ClipboardCopy(*Content);
				}
			}
		})
		.OnPasteFromClipboard_Lambda([TransformType, BoneProxiesView](ESlateTransformComponent::Type InComponent)
		{
			FString Content;
			FPlatformApplicationMisc::ClipboardPaste(Content);

			if(Content.IsEmpty() || TransformType != UBoneProxy::TransformType_Bone)
			{
				return;
			}

			FScopedTransaction Transaction(LOCTEXT("PasteTransform", "Paste Transform"));
			
			static const FName LocationPropertyName = GET_MEMBER_NAME_CHECKED(UBoneProxy, Location);
			static const FName RotationPropertyName = GET_MEMBER_NAME_CHECKED(UBoneProxy, Rotation);
			static const FName ScalePropertyName = GET_MEMBER_NAME_CHECKED(UBoneProxy, Scale);

			for (UBoneProxy* BoneProxy : BoneProxiesView)
			{
				if (!BoneProxy->bIsTransformEditable)
				{
					continue; // Skip non editable bones
				}
				if (const UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
				{
					Component->PreviewInstance->SetFlags(RF_Transactional);
					Component->PreviewInstance->Modify();
					
					switch(InComponent)
					{
						case ESlateTransformComponent::Location:
						{
							FVector Data = BoneProxy->Location;
							if (GetDataFromContent(Content, Data))
							{
								BoneProxy->OnPreEditChange(LocationPropertyName);
								BoneProxy->Location = Data;
								BoneProxy->OnPostEditChangeProperty(LocationPropertyName);
							}
							break;
						}
						case ESlateTransformComponent::Rotation:
						{
							FRotator Data = BoneProxy->Rotation;
							if (GetDataFromContent(Content, Data))
							{
								BoneProxy->OnPreEditChange(RotationPropertyName);
								BoneProxy->Rotation = Data;
								BoneProxy->OnPostEditChangeProperty(RotationPropertyName);
							}
							break;
						}
						case ESlateTransformComponent::Scale:
						{
							FVector Data = BoneProxy->Scale;
							if (GetDataFromContent(Content, Data))
							{
								BoneProxy->OnPreEditChange(ScalePropertyName);
								BoneProxy->Scale = Data;
								BoneProxy->OnPostEditChangeProperty(ScalePropertyName);
							}
							break;
						}
						case ESlateTransformComponent::Max:
						default:
						{
							FEulerTransform Data = FEulerTransform::Identity;
							if (GetDataFromContent(Content, Data))
							{
								BoneProxy->OnPreEditChange(LocationPropertyName);
								BoneProxy->Location = Data.GetLocation();
								BoneProxy->OnPostEditChangeProperty(LocationPropertyName);
								
								BoneProxy->OnPreEditChange(RotationPropertyName);
								BoneProxy->Rotation = Data.Rotator();
								BoneProxy->OnPostEditChangeProperty(RotationPropertyName);
								
								BoneProxy->OnPreEditChange(ScalePropertyName);
								BoneProxy->Scale = Data.GetScale3D();
								BoneProxy->OnPostEditChangeProperty(ScalePropertyName);
							}
							break;
						}
					}
				}
			}
		})
		.DiffersFromDefault_Lambda([TransformType, BoneProxiesView](ESlateTransformComponent::Type InComponent) -> bool
		{
			for(UBoneProxy* BoneProxy : BoneProxiesView)
			{
				if(BoneProxy->DiffersFromDefault(InComponent, TransformType))
				{
					return true;
				}
			}
			return false;
		})
		.OnResetToDefault_Lambda([TransformType, BoneProxiesView](ESlateTransformComponent::Type InComponent)
		{
			FScopedTransaction Transaction(LOCTEXT("ResetToDefault", "Reset to Default"));
			
			for(UBoneProxy* BoneProxy : BoneProxiesView)
			{
				BoneProxy->ResetToDefault(InComponent, TransformType);
			}
		});

		TransformWidgetArgs.Visibility_Lambda([TransformChoiceWidget, TransformType]() -> EVisibility
		{
			return TransformChoiceWidget->HasValue(TransformType) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		if(bIsEditingEnabled)
		{
			constexpr bool bNonTransactional = false;
			constexpr bool bTransactional = true;
			TransformWidgetArgs.OnBeginSliderMovement_Static(&UBoneProxy::OnSliderMovementStateChanged, 0.0, UBoneProxy::ESliderMovementState::Begin, BoneProxiesView);
			TransformWidgetArgs.OnEndSliderMovement_Static(&UBoneProxy::OnSliderMovementStateChanged, UBoneProxy::ESliderMovementState::End, BoneProxiesView);
			TransformWidgetArgs.OnNumericValueChanged_Static(&UBoneProxy::OnMultiNumericValueChanged, ETextCommit::Default, bNonTransactional, TransformType, BoneProxiesView);
			TransformWidgetArgs.OnNumericValueCommitted_Static (&UBoneProxy::OnMultiNumericValueChanged, bTransactional, TransformType, BoneProxiesView);
		}

		SAdvancedTransformInputBox<FEulerTransform>::ConstructGroupedTransformRows(
			CategoryBuilder, 
			ButtonLabels[TransformIndex], 
			ButtonTooltips[TransformIndex], 
			TransformWidgetArgs);
	}
}

bool FBoneProxyDetailsCustomization::IsResetLocationVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies)
{
	for (UBoneProxy* BoneProxy : InBoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			if (FAnimNode_ModifyBone* ModifyBone = Component->PreviewInstance->FindModifiedBone(BoneProxy->BoneName))
			{
				if (ModifyBone->Translation != FVector::ZeroVector && BoneProxy->bIsTransformEditable)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FBoneProxyDetailsCustomization::IsResetRotationVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies)
{
	for (UBoneProxy* BoneProxy : InBoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			if (FAnimNode_ModifyBone* ModifyBone = Component->PreviewInstance->FindModifiedBone(BoneProxy->BoneName))
			{
				if (ModifyBone->Rotation != FRotator::ZeroRotator && BoneProxy->bIsTransformEditable)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FBoneProxyDetailsCustomization::IsResetScaleVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies)
{
	for (UBoneProxy* BoneProxy : InBoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			if (FAnimNode_ModifyBone* ModifyBone = Component->PreviewInstance->FindModifiedBone(BoneProxy->BoneName))
			{
				if (ModifyBone->Scale != FVector(1.0f) && BoneProxy->bIsTransformEditable)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FBoneProxyDetailsCustomization::HandleResetLocation(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies)
{
	FScopedTransaction Transaction(LOCTEXT("ResetLocation", "Reset Location"));

	for (UBoneProxy* BoneProxy : InBoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			BoneProxy->Modify();
			Component->PreviewInstance->Modify();

			FAnimNode_ModifyBone& ModifyBone = Component->PreviewInstance->ModifyBone(BoneProxy->BoneName);
			ModifyBone.Translation = FVector::ZeroVector;

			RemoveUnnecessaryModifications(Component, ModifyBone);
		}
	}
}

void FBoneProxyDetailsCustomization::HandleResetRotation(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies)
{
	FScopedTransaction Transaction(LOCTEXT("ResetRotation", "Reset Rotation"));

	for (UBoneProxy* BoneProxy : InBoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			BoneProxy->Modify();
			Component->PreviewInstance->Modify();

			FAnimNode_ModifyBone& ModifyBone = Component->PreviewInstance->ModifyBone(BoneProxy->BoneName);
			ModifyBone.Rotation = FRotator::ZeroRotator;

			RemoveUnnecessaryModifications(Component, ModifyBone);
		}
	}
}

void FBoneProxyDetailsCustomization::HandleResetScale(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies)
{
	FScopedTransaction Transaction(LOCTEXT("ResetScale", "Reset Scale"));

	for (UBoneProxy* BoneProxy : InBoneProxies)
	{
		if (UDebugSkelMeshComponent* Component = BoneProxy->SkelMeshComponent.Get())
		{
			BoneProxy->Modify();
			Component->PreviewInstance->Modify();

			FAnimNode_ModifyBone& ModifyBone = Component->PreviewInstance->ModifyBone(BoneProxy->BoneName);
			ModifyBone.Scale = FVector(1.0f);

			RemoveUnnecessaryModifications(Component, ModifyBone);
		}
	}
}

void FBoneProxyDetailsCustomization::RemoveUnnecessaryModifications(UDebugSkelMeshComponent* Component, FAnimNode_ModifyBone& ModifyBone)
{
	if (ModifyBone.Translation == FVector::ZeroVector && ModifyBone.Rotation == FRotator::ZeroRotator && ModifyBone.Scale == FVector(1.0f))
	{
		Component->PreviewInstance->RemoveBoneModification(ModifyBone.BoneToModify.BoneName);
	}
}

#undef LOCTEXT_NAMESPACE
