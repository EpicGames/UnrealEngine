// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Containers/ArrayView.h"

class IPropertyHandle;
class UBoneProxy;
class UDebugSkelMeshComponent;
struct FAnimNode_ModifyBone;

class FBoneProxyDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FBoneProxyDetailsCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	/** Handle reset to defaults visibility */
	bool IsResetLocationVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies);

	/** Handle reset to defaults visibility */
	bool IsResetRotationVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies);

	/** Handle reset to defaults visibility */
	bool IsResetScaleVisible(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies);

	/** Handle resetting defaults */
	void HandleResetLocation(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies);

	/** Handle resetting defaults */
	void HandleResetRotation(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies);

	/** Handle resetting defaults */
	void HandleResetScale(TSharedPtr<IPropertyHandle> InPropertyHandle, TArrayView<UBoneProxy*> InBoneProxies);

	/** Remove any modification node if it has no effect */
	void RemoveUnnecessaryModifications(UDebugSkelMeshComponent* Component, FAnimNode_ModifyBone& ModifyBone);

private:
	/** Cached array of selected bone proxies */
	TArray<UBoneProxy*> BoneProxies;
};
