// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditableSkeleton.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "Animation/PoseAsset.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ISkeletonTree.h"
#include "SSkeletonTree.h"
#include "ScopedTransaction.h"
#include "Factories.h"
#include "Animation/BlendProfile.h"
#include "IPersonaPreviewScene.h"
#include "SBlendProfilePicker.h"
#include "AssetNotifications.h"
#include "BlueprintActionDatabase.h"
#include "IAssetFamily.h"
#include "PersonaModule.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SSkeletonWidget.h"
#include "Engine/DataAsset.h"
#include "Animation/PreviewCollectionInterface.h"
#include "HAL/PlatformApplicationMisc.h"

const FString FEditableSkeleton::SocketCopyPasteHeader = TEXT("SocketCopyPasteBuffer");

#define LOCTEXT_NAMESPACE "EditableSkeleton"

/** Constructs sockets from clipboard data */

class FSocketTextObjectFactory : public FCustomizableTextObjectFactory
{
public:
	typedef TFunction<void(USkeletalMeshSocket*, USkeletalMesh*)> FNameCorrectionFunction;

	FSocketTextObjectFactory(USkeleton* InSkeleton, USkeletalMesh* InSkeletalMesh, FName InPasteBone, FNameCorrectionFunction InNameCorrector)
		: FCustomizableTextObjectFactory(GWarn)
		, PasteBone(InPasteBone)
		, Skeleton(InSkeleton)
		, SkeletalMesh(InSkeletalMesh)
		, NameCorrector(InNameCorrector)
	{
		check(Skeleton);
	};

	// Pointer back to the outside world that will hold the final imported socket
	TArray<USkeletalMeshSocket*> CreatedSockets;

private:

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override { return true; }

	virtual void ProcessConstructedObject(UObject* CreatedObject)
	{
		USkeletalMeshSocket* NewSocket = CastChecked<USkeletalMeshSocket>(CreatedObject);
		CreatedSockets.Add(NewSocket);

		const FReferenceSkeleton* RefSkel = nullptr;

		NameCorrector(NewSocket, SkeletalMesh);

		if (USkeletalMesh* SocketMesh = Cast<USkeletalMesh>(NewSocket->GetOuter()))
		{
			SocketMesh->GetMeshOnlySocketList().Add(NewSocket);
			RefSkel = &SocketMesh->GetRefSkeleton();
		}
		else if (USkeleton* SocketSkeleton = Cast<USkeleton>(NewSocket->GetOuter()))
		{
			SocketSkeleton->Sockets.Add(NewSocket);
			RefSkel = &SocketSkeleton->GetReferenceSkeleton();
		}
		else
		{
			check(false) //Unknown socket outer
		}

		if (!PasteBone.IsNone())
		{
			// Override the bone name to the one we pasted to
			NewSocket->BoneName = PasteBone;
		}
		else
		{
			//Validate BoneName
			if (RefSkel->FindBoneIndex(NewSocket->BoneName) == INDEX_NONE)
			{
				NewSocket->BoneName = RefSkel->GetBoneName(0);
			}
		}
	}

	virtual void ProcessUnidentifiedLine(const FString& StrLine)
	{
		bool bIsOnSkeleton;
		FParse::Bool(*StrLine, TEXT("IsOnSkeleton="), bIsOnSkeleton);
		bExpectingMeshSocket = !bIsOnSkeleton;
	}

	virtual UObject* GetParentForNewObject(const UClass* ObjClass)
	{
		UObject* Target = (bExpectingMeshSocket && SkeletalMesh) ? (UObject*)SkeletalMesh : (UObject*)Skeleton;
		Target->Modify();
		return Target;
	}

	const FName PasteBone;

	// Track what type of socket we will be processing next
	bool bExpectingMeshSocket;

	// Target for Skeleton Sockets
	USkeleton* Skeleton;

	// Target for Mesh Sockets (could be nullptr)
	USkeletalMesh* SkeletalMesh;

	// Callback so calling code can have a change to correct the newly pasted socket name
	// before it is added to the mesh/skeleton
	FNameCorrectionFunction NameCorrector;

};

FEditableSkeleton::FEditableSkeleton(USkeleton* InSkeleton)
	: Skeleton(InSkeleton)
{
	check(Skeleton);
	Skeleton->SetFlags(RF_Transactional);
}

const USkeleton& FEditableSkeleton::GetSkeleton() const
{
	check(Skeleton);
	return *Skeleton;
}

bool FEditableSkeleton::IsSkeletonValid() const
{
	return Skeleton != nullptr;
}

const TArray<class UBlendProfile*>& FEditableSkeleton::GetBlendProfiles() const
{
	check(Skeleton);
	return Skeleton->BlendProfiles;
}

UBlendProfile* FEditableSkeleton::GetBlendProfile(const FName& NameToUse)
{
	return Skeleton ? Skeleton->GetBlendProfile(NameToUse) : nullptr;
}

UBlendProfile* FEditableSkeleton::CreateNewBlendProfile(const FName& NameToUse)
{
	FScopedTransaction Transaction(LOCTEXT("CreateBlendProfile", "Create Blend Profile"));

	return Skeleton ? Skeleton->CreateNewBlendProfile(NameToUse) : nullptr;
}

void FEditableSkeleton::RemoveBlendProfile(UBlendProfile* InBlendProfile)
{
	if(InBlendProfile && Skeleton)
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveBlendProfile", "Remove Blend Profile"));

		Skeleton->Modify();
		Skeleton->BlendProfiles.Remove(InBlendProfile);
		InBlendProfile->MarkAsGarbage();
	}
}

UBlendProfile* FEditableSkeleton::RenameBlendProfile(const FName& InBlendProfileName, const FName& InNewBlendProfileName)
{
	if (!Skeleton)
	{
		return nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("RenameBlendProfile", "Rename Blend Profile"));

	UBlendProfile* Profile = Skeleton->RenameBlendProfile(InBlendProfileName, InNewBlendProfileName);
	
	if (Profile == nullptr)
	{
		Transaction.Cancel();
	}

	return Profile;
}

void FEditableSkeleton::SetBlendProfileScale(const FName& InBlendProfileName, const FName& InBoneName, float InNewScale, bool bInRecurse)
{
	if (!Skeleton)
	{
		return;
	}

	if (UBlendProfile* BlendProfile = GetBlendProfile(InBlendProfileName))
	{
		FScopedTransaction Transaction(LOCTEXT("SetBlendProfileScale", "Set Blend Profile Scale"));

		BlendProfile->SetFlags(RF_Transactional);
		BlendProfile->Modify();
		const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(InBoneName);
		BlendProfile->SetBoneBlendScale(BoneIndex, InNewScale, bInRecurse, true);
	}
}

void FEditableSkeleton::SetBlendProfileMode(FName InBlendProfileName, EBlendProfileMode ProfileMode)
{
	if (UBlendProfile* BlendProfile = GetBlendProfile(InBlendProfileName))
	{
		FScopedTransaction Transaction(LOCTEXT("SetBlendProfileMode", "Set Blend Profile Mode"));

		const bool bWasBlendMask = BlendProfile->IsBlendMask();
		BlendProfile->SetFlags(RF_Transactional);
		BlendProfile->Modify();
		BlendProfile->Mode = ProfileMode;
		const bool bIsBlendMask = BlendProfile->IsBlendMask();
		// Re-set entry indices from the end to properly handle different default values when changing from/to blend mask
		if (bWasBlendMask != bIsBlendMask)
		{
			for (int32 EntryIndex = BlendProfile->GetNumBlendEntries() - 1; EntryIndex >= 0; --EntryIndex)
			{
				const FBlendProfileBoneEntry& Entry = BlendProfile->ProfileEntries[EntryIndex];
				BlendProfile->SetBoneBlendScale(Entry.BoneReference.BoneName, Entry.BlendScale);
			}
		}
	}
}

void FEditableSkeleton::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Note that this has the potential to set Skeleton to nullptr
	Collector.AddReferencedObject(Skeleton);
}

void FEditableSkeleton::SetBoneTranslationRetargetingMode(FName InBoneName, EBoneTranslationRetargetingMode::Type NewRetargetingMode)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("SetBoneTranslationRetargetingMode", "Set Bone Translation Retargeting Mode"));
	Skeleton->Modify();

	const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(InBoneName);
	Skeleton->SetBoneTranslationRetargetingMode(BoneIndex, NewRetargetingMode);
	FAssetNotifications::SkeletonNeedsToBeSaved(Skeleton);
}

EBoneTranslationRetargetingMode::Type FEditableSkeleton::GetBoneTranslationRetargetingMode(FName InBoneName) const
{
	if (!Skeleton)
	{
		return EBoneTranslationRetargetingMode::Animation; // Arbitrary return value
	}
	const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(InBoneName);
	return Skeleton->GetBoneTranslationRetargetingMode(BoneIndex);
}

void FEditableSkeleton::RefreshBoneTree()
{
	OnTreeRefresh.Broadcast();
}

bool FEditableSkeleton::DoesSocketAlreadyExist(const USkeletalMeshSocket* InSocket, const FText& InSocketName, ESocketParentType SocketParentType, USkeletalMesh* InSkeletalMesh) const
{
	if (!Skeleton)
	{
		return false;
	}

	TArray<TObjectPtr<USkeletalMeshSocket>>* SocketArrayPtr = nullptr;
	if (SocketParentType == ESocketParentType::Mesh && InSkeletalMesh)
	{
		SocketArrayPtr = &InSkeletalMesh->GetMeshOnlySocketList();
	}
	else if(SocketParentType == ESocketParentType::Skeleton)
	{
		SocketArrayPtr = &Skeleton->Sockets;
	}

	if (SocketArrayPtr != nullptr)
	{
		for (auto SocketIt = SocketArrayPtr->CreateConstIterator(); SocketIt; ++SocketIt)
		{
			USkeletalMeshSocket* Socket = *(SocketIt);

			if (InSocket != Socket && Socket->SocketName.ToString() == InSocketName.ToString())
			{
				return true;
			}
		}
	}

	return false;
}

void FEditableSkeleton::GetAssetsContainingCurves(const FName& InContainerName, const TArray<FName>& InNames, TArray<FAssetData>& OutAssets) const
{
	if (!Skeleton)
	{
		OutAssets.Empty();
		return;
	}

	FAssetData SkeletonData(Skeleton);
	const FString CurrentSkeletonName = SkeletonData.GetExportTextName();

	FAssetRegistryModule& AssetModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetModule.Get().GetAssetsByClass(UAnimationAsset::StaticClass()->GetClassPathName(), OutAssets, true);

	GWarn->BeginSlowTask(LOCTEXT("CollectAnimationsTaskDesc", "Collecting assets..."), true);

	for (int32 Idx = OutAssets.Num() - 1; Idx >= 0; --Idx)
	{
		FAssetData& Data = OutAssets[Idx];
		bool bAssetContainsRemovableCurves = false;

		const FString SkeletonDataTag = Data.GetTagValueRef<FString>("Skeleton");
		if (!SkeletonDataTag.IsEmpty() && SkeletonDataTag == CurrentSkeletonName)
		{
			FString CurveData;
			if (!Data.GetTagValue(USkeleton::CurveNameTag, CurveData))
			{
				// This asset is old; it hasn't been loaded since before smartnames were added for curves.
				// unfortunately the only way to delete curves safely is to load old assets to see if they're
				// using the selected name. We only load what we have to here.
				UObject* Asset = Data.GetAsset();
				check(Asset);
				FAssetRegistryTagsContextData TagsContext(Asset, EAssetRegistryTagsCaller::Uncategorized);
				Asset->GetAssetRegistryTags(TagsContext);

				UObject::FAssetRegistryTag* CurveTag = TagsContext.Tags.Find(USkeleton::CurveNameTag);
				if (CurveTag)
				{
					CurveData = CurveTag->Value;
				}
			}

			if(!CurveData.IsEmpty())
			{
				TArray<FString> ParsedCurveNames;
				CurveData.ParseIntoArray(ParsedCurveNames, *USkeleton::CurveTagDelimiter, true);

				for (const FString& CurveString : ParsedCurveNames)
				{
					FName CurveName(*CurveString);
					if (InNames.Contains(CurveName))
					{
						bAssetContainsRemovableCurves = true;
						break;
					}
				}
			}
		}

		if (!bAssetContainsRemovableCurves)
		{
			OutAssets.RemoveAtSwap(Idx,EAllowShrinking::No);
		}
	}

	GWarn->EndSlowTask();
}

FName FEditableSkeleton::GenerateUniqueSocketName( FName InName, USkeletalMesh* InSkeletalMesh )
{
	if(DoesSocketAlreadyExist(nullptr, FText::FromName(InName), ESocketParentType::Skeleton, InSkeletalMesh ) || DoesSocketAlreadyExist(nullptr, FText::FromName(InName), ESocketParentType::Mesh, InSkeletalMesh))
	{
		int32 NewNumber = InName.GetNumber();

		// Increment NewNumber until we have a unique name (potential infinite loop if *all* int32 values are used!)
		while (DoesSocketAlreadyExist(NULL, FText::FromName(FName(InName, NewNumber)), ESocketParentType::Skeleton, InSkeletalMesh) ||
			(InSkeletalMesh ? DoesSocketAlreadyExist(NULL, FText::FromName(FName(InName, NewNumber)), ESocketParentType::Mesh, InSkeletalMesh) : false))
		{
			++NewNumber;
		}

		return FName(InName, NewNumber);
	}
	return InName;
}

static USkeletalMeshSocket* FindSocket(const FName& InSocketName, USkeletalMesh* InSkeletalMesh, USkeleton* InSkeleton)
{
	USkeletalMeshSocket* Socket = nullptr;

	// first check the skeletal mesh as that is the behavior of USkinnedMeshComponent
	if (InSkeletalMesh != nullptr)
	{
		Socket = InSkeletalMesh->FindSocket(InSocketName);
	}

	if (Socket == nullptr && InSkeleton != nullptr)
	{
		Socket = InSkeleton->FindSocket(InSocketName);
	}

	return Socket;
}

void FEditableSkeleton::RenameSocket(const FName OldSocketName, const FName NewSocketName, USkeletalMesh* InSkeletalMesh)
{
	USkeletalMeshSocket* SocketData = FindSocket(OldSocketName, InSkeletalMesh, Skeleton);

	if (SocketData != nullptr)
	{
		const FScopedTransaction Transaction(Skeleton->PreviewAttachedAssetContainer.Num() > 0 ? LOCTEXT("RenameSocketAndMoveAttachments", "Rename Socket And Move Attachments") : LOCTEXT("RenameSocket", "Rename Socket"));
		SocketData->SetFlags(RF_Transactional);	// Undo doesn't work without this!
		SocketData->Modify();

		SocketData->SocketName = NewSocketName;

		bool bSkeletonModified = false;
		for (int AttachedObjectIndex = 0; AttachedObjectIndex < Skeleton->PreviewAttachedAssetContainer.Num(); ++AttachedObjectIndex)
		{
			FPreviewAttachedObjectPair& Pair = Skeleton->PreviewAttachedAssetContainer[AttachedObjectIndex];
			if (Pair.AttachedTo == OldSocketName)
			{
				// Only modify the skeleton if we actually intend to change something.
				if (!bSkeletonModified)
				{
					Skeleton->Modify();
					bSkeletonModified = true;
				}
				Pair.AttachedTo = NewSocketName;
			}

			// push to skeleton trees
			for (TWeakPtr<SSkeletonTree> SkeletonTree : SkeletonTrees)
			{
				if (SkeletonTree.IsValid())
				{
					SkeletonTree.Pin()->PostRenameSocket(Pair.GetAttachedObject(), OldSocketName, Pair.AttachedTo);
				}
			}
		}

		if (InSkeletalMesh != nullptr)
		{
			FPreviewAssetAttachContainer& PreviewAttachedAssetContainer = InSkeletalMesh->GetPreviewAttachedAssetContainer();
			bool bMeshModified = false;
			for (int AttachedObjectIndex = 0; AttachedObjectIndex < PreviewAttachedAssetContainer.Num(); ++AttachedObjectIndex)
			{
				FPreviewAttachedObjectPair& Pair = PreviewAttachedAssetContainer[AttachedObjectIndex];
				if (Pair.AttachedTo == OldSocketName)
				{
					// Only modify the mesh if we actually intend to change something. Avoids dirtying
					// meshes when we don't actually update any data on them. (such as adding a new socket)
					if (!bMeshModified)
					{
						InSkeletalMesh->Modify();
						bMeshModified = true;
					}
					Pair.AttachedTo = NewSocketName;
				}

				for (TWeakPtr<SSkeletonTree> SkeletonTree : SkeletonTrees)
				{
					if (SkeletonTree.IsValid())
					{
						SkeletonTree.Pin()->PostRenameSocket(Pair.GetAttachedObject(), OldSocketName, Pair.AttachedTo);
					}
				}
			}
		}

		OnTreeRefresh.Broadcast();
	}
}

void FEditableSkeleton::SetSocketParent(const FName& SocketName, const FName& NewParentName, USkeletalMesh* InSkeletalMesh, bool bKeepAbsoluteLocation)
{
	if (!Skeleton)
	{
		return;
	}

	USkeletalMeshSocket* Socket = nullptr;
	const FReferenceSkeleton* ReferenceSkeleton = nullptr;

	if (InSkeletalMesh)
	{
		Socket = InSkeletalMesh->FindSocket(SocketName);
		ReferenceSkeleton = &InSkeletalMesh->GetRefSkeleton();
	}
	if (!Socket && Skeleton)
	{
		Socket = Skeleton->FindSocket(SocketName);
		ReferenceSkeleton = &Skeleton->GetReferenceSkeleton();
	}
	if (Socket)
	{
		// Create an undo transaction, re-parent the socket and rebuild the skeleton tree view
		const FScopedTransaction Transaction(LOCTEXT("ReparentSocket", "Re-parent Socket"));

		Socket->SetFlags(RF_Transactional);	// Undo doesn't work without this!
		Socket->Modify();

		if (bKeepAbsoluteLocation)
		{
			const int32 OldBoneIndex = ReferenceSkeleton->FindBoneIndex(Socket->BoneName);
			const int32 NewBoneIndex = ReferenceSkeleton->FindBoneIndex(NewParentName);

			if (OldBoneIndex != NewBoneIndex && OldBoneIndex != INDEX_NONE && NewBoneIndex != INDEX_NONE)
			{
				// Transform the relative positioning of the socket from the space of the old parent bone to the space of the
				// new parent bone.
				const FTransform OldRefTransform = ReferenceSkeleton->GetBoneAbsoluteTransform(OldBoneIndex);
				const FTransform NewRefTransform = ReferenceSkeleton->GetBoneAbsoluteTransform(NewBoneIndex);

				Socket->SetSocketLocalTransform(Socket->GetSocketLocalTransform() * OldRefTransform * NewRefTransform.Inverse());
			}
		}
		
		Socket->BoneName = NewParentName;

		for (TWeakPtr<SSkeletonTree> SkeletonTree : SkeletonTrees)
		{
			if (SkeletonTree.IsValid())
			{
				SkeletonTree.Pin()->PostSetSocketParent();
			}
		}

		OnTreeRefresh.Broadcast();
	}
}

void FEditableSkeleton::HandlePasteSockets(const FName& InBoneName, USkeletalMesh* InSkeletalMesh)
{
	if (!Skeleton)
	{
		return;
	}

	FString PasteString;
	FPlatformApplicationMisc::ClipboardPaste(PasteString);
	const TCHAR* PastePtr = *PasteString;

	FString PasteLine;
	FParse::Line(&PastePtr, PasteLine);

	if (PasteLine == SocketCopyPasteHeader)
	{
		const FScopedTransaction Transaction(LOCTEXT("PasteSockets", "Paste sockets"));

		int32 NumSocketsToPaste;
		FParse::Line(&PastePtr, PasteLine);	// Need this to advance PastePtr, for multiple sockets
		FParse::Value(*PasteLine, TEXT("NumSockets="), NumSocketsToPaste);

		FSocketTextObjectFactory TextObjectFactory( Skeleton,
		                                            InSkeletalMesh,
		                                            InBoneName,
		                                            [this](USkeletalMeshSocket* NewSocket, USkeletalMesh* InLocalSkeletalMesh)
		                                            {
		                                            	NewSocket->SocketName = GenerateUniqueSocketName(NewSocket->SocketName, InLocalSkeletalMesh);
		                                            } );
		TextObjectFactory.ProcessBuffer(nullptr, RF_Transactional, PastePtr);
	}
}

USkeletalMeshSocket* FEditableSkeleton::AddSocket(const FName& InBoneName)
{
	USkeletalMeshSocket* NewSocket = HandleAddSocket(InBoneName);
	OnTreeRefresh.Broadcast();
	return NewSocket;
}

USkeletalMeshSocket* FEditableSkeleton::HandleAddSocket(const FName& InBoneName)
{
	if (!Skeleton)
	{
		return nullptr;
	}
	const FScopedTransaction Transaction(LOCTEXT("AddSocket", "Add Socket to Skeleton"));
	Skeleton->Modify();

	USkeletalMeshSocket* NewSocket = NewObject<USkeletalMeshSocket>(Skeleton);
	check(NewSocket);

	NewSocket->BoneName = InBoneName;
	FString SocketName = NewSocket->BoneName.ToString() + LOCTEXT("SocketPostfix", "Socket").ToString();
	NewSocket->SocketName = GenerateUniqueSocketName(*SocketName, nullptr);

	Skeleton->Sockets.Add(NewSocket);
	return NewSocket;
}

bool FEditableSkeleton::HandleAddVirtualBone(const FName SourceBoneName, const FName TargetBoneName)
{
	FName Dummy;
	return HandleAddVirtualBone(SourceBoneName, TargetBoneName, Dummy);
}

bool FEditableSkeleton::HandleAddVirtualBone(const FName SourceBoneName, const FName TargetBoneName, FName& NewVirtualBoneName)
{
	if (!Skeleton)
	{
		return false;
	}
	FScopedTransaction Transaction(LOCTEXT("AddVirtualBone", "Add Virtual Bone to Skeleton"));
	const bool Success = Skeleton->AddNewVirtualBone(SourceBoneName, TargetBoneName, NewVirtualBoneName);
	if (!Success)
	{
		Transaction.Cancel();
	}
	else
	{
		OnTreeRefresh.Broadcast();
	}
	return Success;
}

void FEditableSkeleton::HandleCustomizeSocket(USkeletalMeshSocket* InSocketToCustomize, USkeletalMesh* InSkeletalMesh)
{
	if (InSkeletalMesh)
	{
		const FScopedTransaction Transaction(LOCTEXT("CreateMeshSocket", "Create Mesh Socket"));
		InSkeletalMesh->Modify();

		USkeletalMeshSocket* NewSocket = NewObject<USkeletalMeshSocket>(InSkeletalMesh);
		check(NewSocket);

		NewSocket->BoneName = InSocketToCustomize->BoneName;
		NewSocket->SocketName = InSocketToCustomize->SocketName;
		NewSocket->RelativeLocation = InSocketToCustomize->RelativeLocation;
		NewSocket->RelativeRotation = InSocketToCustomize->RelativeRotation;
		NewSocket->RelativeScale = InSocketToCustomize->RelativeScale;

		InSkeletalMesh->GetMeshOnlySocketList().Add(NewSocket);
	}
}

void FEditableSkeleton::HandlePromoteSocket(USkeletalMeshSocket* InSocketToPromote)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("PromoteSocket", "Promote Socket"));
	Skeleton->Modify();

	USkeletalMeshSocket* NewSocket = NewObject<USkeletalMeshSocket>(Skeleton);
	check(NewSocket);

	NewSocket->BoneName = InSocketToPromote->BoneName;
	NewSocket->SocketName = InSocketToPromote->SocketName;
	NewSocket->RelativeLocation = InSocketToPromote->RelativeLocation;
	NewSocket->RelativeRotation = InSocketToPromote->RelativeRotation;
	NewSocket->RelativeScale = InSocketToPromote->RelativeScale;

	Skeleton->Sockets.Add(NewSocket);
}

void FEditableSkeleton::HandleRemoveAllAssets(TSharedPtr<IPersonaPreviewScene> InPreviewScene)
{
	if (!Skeleton)
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("AttachedAssetRemoveUndo", "Remove All Attached Assets"));
	Skeleton->Modify();

	DeleteAttachedObjects(Skeleton->PreviewAttachedAssetContainer, InPreviewScene);

	USkeletalMesh* SkeletalMesh = InPreviewScene->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	if (SkeletalMesh)
	{
		SkeletalMesh->Modify();
		DeleteAttachedObjects(SkeletalMesh->GetPreviewAttachedAssetContainer(), InPreviewScene);
	}
}

void FEditableSkeleton::DeleteAttachedObjects(FPreviewAssetAttachContainer& AttachedAssets, TSharedPtr<IPersonaPreviewScene> InPreviewScene)
{
	if (InPreviewScene.IsValid())
	{
		for (auto Iter = AttachedAssets.CreateIterator(); Iter; ++Iter)
		{
			FPreviewAttachedObjectPair& Pair = (*Iter);
			InPreviewScene->RemoveAttachedObjectFromPreviewComponent(Pair.GetAttachedObject(), Pair.AttachedTo);
		}
	}

	AttachedAssets.ClearAllAttachedObjects();
}

USkeletalMeshSocket* FEditableSkeleton::DuplicateSocket(const FSelectedSocketInfo& SocketInfoToDuplicate, const FName& NewParentBoneName, USkeletalMesh* InSkeletalMesh)
{
	if (!Skeleton)
	{
		return nullptr;
	}
	check(SocketInfoToDuplicate.Socket);

	const FScopedTransaction Transaction(LOCTEXT("CopySocket", "Copy Socket"));

	USkeletalMeshSocket* NewSocket;

	bool bModifiedSkeleton = false;

	if (SocketInfoToDuplicate.bSocketIsOnSkeleton)
	{
		Skeleton->Modify();
		bModifiedSkeleton = true;
		NewSocket = NewObject<USkeletalMeshSocket>(Skeleton);
	}
	else if (InSkeletalMesh)
	{
		InSkeletalMesh->Modify();
		NewSocket = NewObject<USkeletalMeshSocket>(InSkeletalMesh);
	}
	else
	{
		// Original socket was on the mesh, but we have no mesh. Huh?
		check(0);
		return nullptr;
	}

	NewSocket->SocketName = GenerateUniqueSocketName(SocketInfoToDuplicate.Socket->SocketName, InSkeletalMesh);
	NewSocket->BoneName = NewParentBoneName != "" ? NewParentBoneName : SocketInfoToDuplicate.Socket->BoneName;
	NewSocket->RelativeLocation = SocketInfoToDuplicate.Socket->RelativeLocation;
	NewSocket->RelativeRotation = SocketInfoToDuplicate.Socket->RelativeRotation;
	NewSocket->RelativeScale = SocketInfoToDuplicate.Socket->RelativeScale;

	if (SocketInfoToDuplicate.bSocketIsOnSkeleton)
	{
		Skeleton->Sockets.Add(NewSocket);
	}
	else if (InSkeletalMesh)
	{
		InSkeletalMesh->GetMeshOnlySocketList().Add(NewSocket);
	}

	// Duplicated attached assets
	int32 NumExistingAttachedObjects = Skeleton->PreviewAttachedAssetContainer.Num();
	for (int AttachedObjectIndex = 0; AttachedObjectIndex < NumExistingAttachedObjects; ++AttachedObjectIndex)
	{
		FPreviewAttachedObjectPair& Pair = Skeleton->PreviewAttachedAssetContainer[AttachedObjectIndex];
		if (Pair.AttachedTo == SocketInfoToDuplicate.Socket->SocketName)
		{
			if (!bModifiedSkeleton)
			{
				bModifiedSkeleton = true;
				Skeleton->Modify();
			}

			for (TWeakPtr<SSkeletonTree> SkeletonTree : SkeletonTrees)
			{
				if (SkeletonTree.IsValid())
				{
					SkeletonTree.Pin()->PostDuplicateSocket(Pair.GetAttachedObject(), NewSocket->SocketName);
				}
			}

			// should be safe to call this even though we are growing the PreviewAttachedAssetContainer array as we cache the array count
			Skeleton->PreviewAttachedAssetContainer.AddUniqueAttachedObject(Pair.GetAttachedObject(), NewSocket->SocketName);
		}
	}

	OnTreeRefresh.Broadcast();

	return NewSocket;
}

int32 FEditableSkeleton::ValidatePreviewAttachedObjects()
{
	if (!Skeleton)
	{
		return 0;
	}
	return Skeleton->ValidatePreviewAttachedObjects();
}

void FEditableSkeleton::RecreateBoneTree(USkeletalMesh* NewPreviewMesh)
{
	if (!Skeleton)
	{
		return;
	}
	Skeleton->RecreateBoneTree(NewPreviewMesh);
}

void FEditableSkeleton::HandleAttachAssets(const TArray<UObject*>& InObjects, const FName& InAttachToName, bool bAttachToMesh, TSharedPtr<IPersonaPreviewScene> InPreviewScene)
{
	for (auto Iter = InObjects.CreateConstIterator(); Iter; ++Iter)
	{
		UObject* Object = (*Iter);

		if (bAttachToMesh)
		{
			USkeletalMesh* SkeletalMesh = InPreviewScene->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
			if (SkeletalMesh != nullptr)
			{
				FScopedTransaction Transaction(LOCTEXT("DragDropAttachMeshUndo", "Attach Assets to Mesh"));
				SkeletalMesh->Modify();
				if (InPreviewScene.IsValid())
				{
					InPreviewScene->AttachObjectToPreviewComponent(Object, InAttachToName);
				}
				SkeletalMesh->GetPreviewAttachedAssetContainer().AddAttachedObject(Object, InAttachToName);
			}
		}
		else if (Skeleton)
		{
			FScopedTransaction Transaction(LOCTEXT("DragDropAttachSkeletonUndo", "Attach Assets to Skeleton"));
			Skeleton->Modify();
			if (InPreviewScene.IsValid())
			{
				InPreviewScene->AttachObjectToPreviewComponent(Object, InAttachToName);
			}
			Skeleton->PreviewAttachedAssetContainer.AddAttachedObject(Object, InAttachToName);
		}
	}
}

void FEditableSkeleton::SetBoneTranslationRetargetingModeRecursive(const TArray<FName>& InBoneNames, EBoneTranslationRetargetingMode::Type NewRetargetingMode)
{
	if (Skeleton && InBoneNames.Num())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetBoneTranslationRetargetingModeRecursive", "Set Bone Translation Retargeting Mode Recursive"));
		Skeleton->Modify();

		for (const FName& BoneName : InBoneNames)
		{
			int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
			Skeleton->SetBoneTranslationRetargetingMode(BoneIndex, NewRetargetingMode, true);
		}

		FAssetNotifications::SkeletonNeedsToBeSaved(Skeleton);
	}
}

void FEditableSkeleton::HandleDeleteAttachedAssets(const TArray<FPreviewAttachedObjectPair>& InAttachedObjects, TSharedPtr<IPersonaPreviewScene> InPreviewScene)
{
	if (Skeleton && InAttachedObjects.Num() > 0)
	{
		Skeleton->Modify();
		USkeletalMesh* SkeletalMesh = InPreviewScene->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
		if (SkeletalMesh != nullptr)
		{
			SkeletalMesh->Modify();
		}

		for (const FPreviewAttachedObjectPair& AttachedObject : InAttachedObjects)
		{
			Skeleton->PreviewAttachedAssetContainer.RemoveAttachedObject(AttachedObject.GetAttachedObject(), AttachedObject.AttachedTo);

			if (SkeletalMesh != nullptr)
			{
				SkeletalMesh->GetPreviewAttachedAssetContainer().RemoveAttachedObject(AttachedObject.GetAttachedObject(), AttachedObject.AttachedTo);

				if (InPreviewScene.IsValid())
				{
					InPreviewScene->RemoveAttachedObjectFromPreviewComponent(AttachedObject.GetAttachedObject(), AttachedObject.AttachedTo);
				}
			}
		}
	}
}

void FEditableSkeleton::HandleDeleteSockets(const TArray<FSelectedSocketInfo>& InSocketInfo, TSharedPtr<IPersonaPreviewScene> InPreviewScene)
{
	if (!Skeleton)
	{
		return;
	}
	for (const FSelectedSocketInfo& SocketInfo : InSocketInfo)
	{
		FName SocketName = SocketInfo.Socket->SocketName;

		if (SocketInfo.bSocketIsOnSkeleton)
		{
			Skeleton->Modify();
			Skeleton->Sockets.Remove(SocketInfo.Socket);
		}
		else
		{
			USkeletalMesh* SkeletalMesh = InPreviewScene->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
			if (SkeletalMesh != nullptr)
			{
				UObject* Object = SkeletalMesh->GetPreviewAttachedAssetContainer().GetAttachedObjectByAttachName(SocketName);
				if (Object)
				{
					SkeletalMesh->Modify();
					SkeletalMesh->GetPreviewAttachedAssetContainer().RemoveAttachedObject(Object, SocketName);
					if (InPreviewScene.IsValid())
					{
						InPreviewScene->RemoveAttachedObjectFromPreviewComponent(Object, SocketName);
					}
				}

				SkeletalMesh->Modify();
				SkeletalMesh->GetMeshOnlySocketList().Remove(SocketInfo.Socket);
			}
		}

		// Remove attached assets
		while (UObject* Object = Skeleton->PreviewAttachedAssetContainer.GetAttachedObjectByAttachName(SocketName))
		{
			Skeleton->Modify();
			Skeleton->PreviewAttachedAssetContainer.RemoveAttachedObject(Object, SocketName);
			if (InPreviewScene.IsValid())
			{
				InPreviewScene->RemoveAttachedObjectFromPreviewComponent(Object, SocketName);
			}
		}
	}

	OnTreeRefresh.Broadcast();
}

void FEditableSkeleton::HandleDeleteVirtualBones(const TArray<FName>& InVirtualBoneInfo, TSharedPtr<class IPersonaPreviewScene> InPreviewScene)
{
	if (!Skeleton)
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("RemoveVirtualBone", "Remove Virtual Bone from Skeleton"));
	Skeleton->RemoveVirtualBones(InVirtualBoneInfo);

	OnTreeRefresh.Broadcast();
}

bool FEditableSkeleton::DoesVirtualBoneAlreadyExist(const FString& InVBName) const
{
	FName NewVBName = FName(*InVBName);
	for (const FVirtualBone& VB : Skeleton->GetVirtualBones())
	{
		if (VB.VirtualBoneName == NewVBName)
		{
			return true;
		}
	}
	return false;
}

void FEditableSkeleton::RenameVirtualBone(const FName OriginalName, const FName InVBName)
{
	if (!Skeleton)
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("RenameVirtualBone", "Rename Virtual Bone in Skeleton"));
	Skeleton->RenameVirtualBone(OriginalName, InVBName);

	OnTreeRefresh.Broadcast();
}

void FEditableSkeleton::RecursiveSetBlendProfileScales(const FName& InBlendProfileName, const TArray<FName>& InBoneNames, float InScaleToSet)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("SetBlendScalesRecursive", "Recursively Set Blend Profile Scales"));
	Skeleton->Modify();

	for (const FName& BoneName : InBoneNames)
	{
		SetBlendProfileScale(InBlendProfileName, BoneName, InScaleToSet, true);
	}

	FAssetNotifications::SkeletonNeedsToBeSaved(Skeleton);
}

TSharedRef<ISkeletonTree> FEditableSkeleton::CreateSkeletonTree(const FSkeletonTreeArgs& InSkeletonTreeArgs)
{
	// first compact widget tracking array
	SkeletonTrees.RemoveAll([](TWeakPtr<SSkeletonTree> InSkeletonTree) { return !InSkeletonTree.IsValid(); });

	// build new tree
	TSharedRef<SSkeletonTree> SkeletonTree = SNew(SSkeletonTree, AsShared(), InSkeletonTreeArgs);

	OnTreeRefresh.AddSP(&SkeletonTree.Get(), &SSkeletonTree::HandleTreeRefresh);

	SkeletonTrees.Add(SkeletonTree);
	return SkeletonTree;
}

TSharedRef<SWidget> FEditableSkeleton::CreateBlendProfilePicker(const FBlendProfilePickerArgs& InArgs)
{
	// first compact widget tracking array
	BlendProfilePickers.RemoveAll([](TWeakPtr<SBlendProfilePicker> InBlendProfilePicker) { return !InBlendProfilePicker.IsValid(); });

	// build new picker
	TSharedRef<SBlendProfilePicker> BlendProfilePicker = SNew(SBlendProfilePicker, AsShared())
		.InitialProfile(InArgs.InitialProfile)
		.OnBlendProfileSelected(InArgs.OnBlendProfileSelected)
		.AllowNew(InArgs.bAllowNew)
		.AllowClear(InArgs.bAllowClear)
		.AllowModify(InArgs.bAllowModify)
		.SupportedBlendProfileModes(InArgs.SupportedBlendProfileModes)
		.PropertyHandle(InArgs.PropertyHandle);

	BlendProfilePickers.Add(BlendProfilePicker);
	return BlendProfilePicker;
}

int32 FEditableSkeleton::DeleteAnimNotifies(const TArray<FName>& InNotifyNames, bool bDeleteFromAnimations)
{
	if (!Skeleton)
	{
		return 0;
	}
	int32 NumAnimationsModified = 0;
	const FScopedTransaction Transaction(LOCTEXT("DeleteAnimNotify", "Delete Anim Notify"));
	Skeleton->Modify();

	for (FName Notify : InNotifyNames)
	{
		Skeleton->AnimationNotifies.Remove(Notify);
	}

	if(bDeleteFromAnimations)
	{
		TArray<FAssetData> CompatibleAnimSequences;
		GetCompatibleAnimSequences(CompatibleAnimSequences);

		for (int32 AssetIndex = 0; AssetIndex < CompatibleAnimSequences.Num(); ++AssetIndex)
		{
			const FAssetData& PossibleAnimSequence = CompatibleAnimSequences[AssetIndex];
			if(UObject* LoadedAsset = PossibleAnimSequence.GetAsset())
			{
				UAnimSequenceBase* Sequence = CastChecked<UAnimSequenceBase>(LoadedAsset);

				if (Sequence->RemoveNotifies(InNotifyNames))
				{
					++NumAnimationsModified;
				}
			}
		}
	}

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.ClearAssetActions(UAnimBlueprint::StaticClass());
	ActionDatabase.RefreshClassActions(UAnimBlueprint::StaticClass());
	OnNotifiesChanged.Broadcast();

	return NumAnimationsModified;
}

int32 FEditableSkeleton::DeleteSyncMarkers(const TArray<FName>& InSyncMarkerNames, bool bDeleteFromAnimations)
{
	if (!Skeleton)
	{
		return 0;
	}
	int32 NumAnimationsModified = 0;
	const FScopedTransaction Transaction(LOCTEXT("DeleteSyncMarkers", "Delete Sync Markers"));
	Skeleton->Modify();

	for (FName Marker : InSyncMarkerNames)
	{
		Skeleton->RemoveMarkerName(Marker);
	}

	if(bDeleteFromAnimations)
	{
		TArray<FAssetData> CompatibleAnimSequences;
		GetCompatibleAnimSequences(CompatibleAnimSequences);

		for (int32 AssetIndex = 0; AssetIndex < CompatibleAnimSequences.Num(); ++AssetIndex)
		{
			const FAssetData& PossibleAnimSequence = CompatibleAnimSequences[AssetIndex];
			if(UObject* LoadedAsset = PossibleAnimSequence.GetAsset())
			{
				if(UAnimSequence* Sequence = Cast<UAnimSequence>(LoadedAsset))
				{
					if (Sequence->RemoveSyncMarkers(InSyncMarkerNames))
					{
						++NumAnimationsModified;
					}
				}
			}
		}
	}
	
	OnNotifiesChanged.Broadcast();

	return NumAnimationsModified;
}

void FEditableSkeleton::AddNotify(FName NewName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("AddNewNotifyToSkeleton", "Add New Anim Notify To Skeleton"));
	Skeleton->Modify();
	Skeleton->AddNewAnimationNotify(NewName);

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.ClearAssetActions(UAnimBlueprint::StaticClass());
	ActionDatabase.RefreshClassActions(UAnimBlueprint::StaticClass());
	OnNotifiesChanged.Broadcast();
}

void FEditableSkeleton::AddSyncMarker(FName NewName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("AddNewSyncMarkerToSkeleton", "Add New Sync Marker To Skeleton"));
	Skeleton->Modify();
	Skeleton->RegisterMarkerName(NewName);

	OnNotifiesChanged.Broadcast();
}

int32 FEditableSkeleton::RenameNotify(const FName NewName, const FName OldName, bool bRenameInAnimations)
{
	if (!Skeleton)
	{
		return 0;
	}
	int32 NumAnimationsModified = 0;
	int32 Index = Skeleton->AnimationNotifies.IndexOfByKey(OldName);
	if(Index != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameAnimNotify", "Rename Anim Notify"));
		Skeleton->Modify();

		Skeleton->AnimationNotifies[Index] = NewName;

		if(bRenameInAnimations)
		{
			TArray<FAssetData> CompatibleAnimSequences;
			GetCompatibleAnimSequences(CompatibleAnimSequences);
			
			for (int32 AssetIndex = 0; AssetIndex < CompatibleAnimSequences.Num(); ++AssetIndex)
			{
				const FAssetData& PossibleAnimSequence = CompatibleAnimSequences[AssetIndex];
				if(UObject* Asset = PossibleAnimSequence.GetAsset())
				{
					UAnimSequenceBase* Sequence = Cast<UAnimSequenceBase>(Asset);

					bool SequenceModified = false;
					for (int32 NotifyIndex = Sequence->Notifies.Num() - 1; NotifyIndex >= 0; --NotifyIndex)
					{
						FAnimNotifyEvent& AnimNotify = Sequence->Notifies[NotifyIndex];
						if (OldName == AnimNotify.NotifyName)
						{
							if (!SequenceModified)
							{
								Sequence->Modify();
								++NumAnimationsModified;
								SequenceModified = true;
							}
							AnimNotify.NotifyName = NewName;
						}
					}

					if (SequenceModified)
					{
						Sequence->MarkPackageDirty();
					}
				}
			}
		}

		FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
		ActionDatabase.ClearAssetActions(UAnimBlueprint::StaticClass());
		ActionDatabase.RefreshClassActions(UAnimBlueprint::StaticClass());
		OnNotifiesChanged.Broadcast();
	}

	return NumAnimationsModified;
}

int32 FEditableSkeleton::RenameSyncMarker(const FName NewName, const FName OldName, bool bRenameInAnimations)
{
	if (!Skeleton)
	{
		return 0;
	}
	int32 NumAnimationsModified = 0;
	int32 Index = Skeleton->ExistingMarkerNames.IndexOfByKey(OldName);
	if(Index != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameSyncMarker", "Rename Sync Marker"));
		Skeleton->Modify();
		Skeleton->ExistingMarkerNames[Index] = NewName;

		if(bRenameInAnimations)
		{
			TArray<FAssetData> CompatibleAnimSequences;
			GetCompatibleAnimSequences(CompatibleAnimSequences);

			for (int32 AssetIndex = 0; AssetIndex < CompatibleAnimSequences.Num(); ++AssetIndex)
			{
				const FAssetData& PossibleAnimSequence = CompatibleAnimSequences[AssetIndex];
				if(UObject* Asset = PossibleAnimSequence.GetAsset())
				{
					if(UAnimSequence* Sequence = Cast<UAnimSequence>(Asset))
					{
						bool SequenceModified = false;
						for (int32 MarkerIndex = Sequence->AuthoredSyncMarkers.Num() - 1; MarkerIndex >= 0; --MarkerIndex)
						{
							FAnimSyncMarker& AnimSyncMarker = Sequence->AuthoredSyncMarkers[MarkerIndex];
							if (OldName == AnimSyncMarker.MarkerName)
							{
								if (!SequenceModified)
								{
									Sequence->Modify();
									++NumAnimationsModified;
									SequenceModified = true;
								}
								AnimSyncMarker.MarkerName = NewName;
							}
						}

						if (SequenceModified)
						{
							Sequence->MarkPackageDirty();
						}
					}
				}
			}
		}

		OnNotifiesChanged.Broadcast();
	}

	return NumAnimationsModified;
}

void FEditableSkeleton::BroadcastNotifyChanged()
{
	OnNotifiesChanged.Broadcast();
}

void FEditableSkeleton::GetCompatibleAnimSequences(TArray<struct FAssetData>& OutAssets)
{
	if (!Skeleton)
	{
		OutAssets.Empty();
		return;
	}
	//Get the skeleton tag to search for
	FString SkeletonExportName = FAssetData(Skeleton).GetExportTextName();

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssetsByClass(UAnimSequenceBase::StaticClass()->GetClassPathName(), AssetDataList, true);

	OutAssets.Empty(AssetDataList.Num());

	for (int32 AssetIndex = 0; AssetIndex < AssetDataList.Num(); ++AssetIndex)
	{
		const FAssetData& PossibleAnimSequence = AssetDataList[AssetIndex];
		if (SkeletonExportName == PossibleAnimSequence.GetTagValueRef<FString>("Skeleton"))
		{
			OutAssets.Add(PossibleAnimSequence);
		}
	}
}

void FEditableSkeleton::RegisterOnSkeletonHierarchyChanged(const USkeleton::FOnSkeletonHierarchyChanged& InDelegate)
{
	if (!Skeleton)
	{
		return;
	}
	Skeleton->RegisterOnSkeletonHierarchyChanged(InDelegate);
}

void FEditableSkeleton::UnregisterOnSkeletonHierarchyChanged(FDelegateUserObject Thing)
{
	if (!Skeleton)
	{
		return;
	}
	Skeleton->UnregisterOnSkeletonHierarchyChanged(Thing);
}

void FEditableSkeleton::RegisterOnNotifiesChanged(const FSimpleMulticastDelegate::FDelegate& InDelegate)
{
	OnNotifiesChanged.Add(InDelegate);
}

void FEditableSkeleton::UnregisterOnNotifiesChanged(FDelegateUserObject Thing)
{
	OnNotifiesChanged.RemoveAll(Thing);
}

void FEditableSkeleton::SetPreviewMesh(class USkeletalMesh* InSkeletalMesh)
{
	if (!Skeleton)
	{
		return;
	}
	if (InSkeletalMesh != Skeleton->GetPreviewMesh())
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeSkeletonPreviewMesh", "Change Skeleton Preview Mesh"));
		Skeleton->SetPreviewMesh(InSkeletalMesh);
	}
}

void FEditableSkeleton::LoadAdditionalPreviewSkeletalMeshes()
{
	if (!Skeleton)
	{
		return;
	}
	Skeleton->LoadAdditionalPreviewSkeletalMeshes();
}

void FEditableSkeleton::SetAdditionalPreviewSkeletalMeshes(class UDataAsset* InPreviewCollectionAsset)
{
	if (!Skeleton)
	{
		return;
	}
	if (!InPreviewCollectionAsset || InPreviewCollectionAsset->GetClass()->ImplementsInterface(UPreviewCollectionInterface::StaticClass()))
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeSkeletonAdditionalMeshes", "Change Skeleton Additional Meshes"));
		Skeleton->SetAdditionalPreviewSkeletalMeshes(InPreviewCollectionAsset);
	}
}

void FEditableSkeleton::RenameRetargetSource(const FName InOldName, const FName InNewName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("RetargetSourceWindow_Rename", "Rename Retarget Source"));

	FReferencePose* Pose = Skeleton->AnimRetargetSources.Find(InOldName);
	if (Pose)
	{
		FReferencePose NewPose = *Pose;
		NewPose.PoseName = InNewName;

		Skeleton->Modify();

		Skeleton->AnimRetargetSources.Remove(InOldName);
		Skeleton->AnimRetargetSources.Add(InNewName, NewPose);

		// need to verify if these pose is used by anybody else
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		TArray<FAssetData> AssetList;
		TMultiMap<FName, FString> TagsAndValues;
		TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSource), InOldName.ToString());
		AssetRegistryModule.Get().GetAssetsByTagValues(TagsAndValues, AssetList);

		// This could get assets of different skeletons here, so trim them
		const FString SkeletonName = FAssetData(Skeleton).GetExportTextName();
		AssetList.RemoveAll([&SkeletonName](const FAssetData& InAssetData)
		{
			FString AssetSkeletonName = InAssetData.GetTagValueRef<FString>(TEXT("Skeleton"));
			return AssetSkeletonName != SkeletonName;
		});
		
		// ask users if they'd like to continue and/or fix up
		if (AssetList.Num() > 0)
		{
			FString ListOfAssets;
			// if users is sure to delete, delete
			for (auto Iter = AssetList.CreateConstIterator(); Iter; ++Iter)
			{
				ListOfAssets += Iter->AssetName.ToString();
				ListOfAssets += TEXT("\n");
			}


			// if so, ask if they'd like us to fix the animations as well
			FString Message = NSLOCTEXT("RetargetSourceEditor", "RetargetSourceRename_FixUpAnimation_Message", "Would you like to fix up the following animation(s)? You'll have to save all the assets in the list.").ToString();
			Message += TEXT("\n");
			Message += ListOfAssets;

			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(Message));

			if (Result == EAppReturnType::Yes)
			{
				// now fix up all assets
				TArray<UObject*> ObjectsToUpdate;
				for (auto Iter = AssetList.CreateConstIterator(); Iter; ++Iter)
				{
					UAnimSequence * AnimSequence = Cast<UAnimSequence>(Iter->GetAsset());
					if (AnimSequence)
					{
						ObjectsToUpdate.Add(AnimSequence);

						AnimSequence->Modify();
						// clear name
						AnimSequence->RetargetSource = InNewName;
					}
				}
			}
		}

		Skeleton->CallbackRetargetSourceChanged();
	}
}

void FEditableSkeleton::AddRetargetSource(const FName& InName, USkeletalMesh* InReferenceMesh)
{
	if (!Skeleton)
	{
		return;
	}
	// need to verify if the name is unique, if not create unique name
	int32 IntSuffix = 1;
	FReferencePose * ExistingPose = NULL;
	FString NewSourceName = TEXT("");
	do
	{
		if (IntSuffix <= 1)
		{
			NewSourceName = FString::Printf(TEXT("%s"), *InName.ToString());
		}
		else
		{
			NewSourceName = FString::Printf(TEXT("%s%d"), *InName.ToString(), IntSuffix);
		}

		ExistingPose = Skeleton->AnimRetargetSources.Find(FName(*NewSourceName));
		IntSuffix++;
	} while (ExistingPose != NULL);

	// add new one
	// remap to skeleton refpose
	// we have to do this whenever skeleton changes
	FReferencePose RefPose;
	RefPose.PoseName = FName(*NewSourceName);
	RefPose.SourceReferenceMesh = InReferenceMesh;

	const FScopedTransaction Transaction(LOCTEXT("RetargetSourceWindow_Add", "Add New Retarget Source"));
	Skeleton->Modify();

	Skeleton->AnimRetargetSources.Add(*NewSourceName, RefPose);
	// ask skeleton to update retarget source for the given name
	Skeleton->UpdateRetargetSource(*NewSourceName);

	Skeleton->CallbackRetargetSourceChanged();
}


void FEditableSkeleton::DeleteRetargetSources(const TArray<FName>& InRetargetSourceNames)
{
	if (!Skeleton)
	{
		return;
	}
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	const FScopedTransaction Transaction(LOCTEXT("RetargetSourceWindow_Delete", "Delete Retarget Source"));
	for (const FName& SourceName : InRetargetSourceNames)
	{
		const FReferencePose* PoseFound = Skeleton->AnimRetargetSources.Find(SourceName);
		if (PoseFound)
		{
			// need to verify if these pose is used by anybody else
			TArray<FAssetData> AssetList;
			TMultiMap<FName, FString> TagsAndValues;
			TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSource), PoseFound->PoseName.ToString());
			AssetRegistryModule.Get().GetAssetsByTagValues(TagsAndValues, AssetList);

			// This could get assets of different skeletons here, so trim them
			const FString SkeletonName = FAssetData(Skeleton).GetExportTextName();
			AssetList.RemoveAll([&SkeletonName](const FAssetData& InAssetData)
			{
				FString AssetSkeletonName = InAssetData.GetTagValueRef<FString>(TEXT("Skeleton"));
				return AssetSkeletonName != SkeletonName;
			});
			
			// ask users if they'd like to continue and/or fix up
			if (AssetList.Num() > 0)
			{
				FString ListOfAssets;
				// if users is sure to delete, delete
				for (auto Iter = AssetList.CreateConstIterator(); Iter; ++Iter)
				{
					ListOfAssets += Iter->AssetName.ToString();
					ListOfAssets += TEXT("\n");
				}

				// ask if they'd like to continue deleting this pose regardless animation references
				FString Message = NSLOCTEXT("RetargetSourceEditor", "RetargetSourceDeleteMessage", "Following animation(s) is(are) referencing this pose. Are you sure if you'd like to delete this pose?").ToString();
				Message += TEXT("\n\n");
				Message += ListOfAssets;

				EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(Message));

				if (Result == EAppReturnType::No)
				{
					continue;
				}

				// if so, ask if they'd like us to fix the animations as well
				Message = NSLOCTEXT("RetargetSourceEditor", "RetargetSourceDelete_FixUpAnimation_Message", "Would you like to fix up the following animation(s)? You'll have to save all the assets in the list.").ToString();
				Message += TEXT("\n");
				Message += ListOfAssets;

				Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(Message));

				if (Result == EAppReturnType::No)
				{
					continue;
				}

				// now fix up all assets
				TArray<UObject *> ObjectsToUpdate;
				for (auto Iter = AssetList.CreateConstIterator(); Iter; ++Iter)
				{
					UAnimSequence * AnimSequence = Cast<UAnimSequence>(Iter->GetAsset());
					if (AnimSequence)
					{
						ObjectsToUpdate.Add(AnimSequence);

						AnimSequence->Modify();
						// clear name
						AnimSequence->RetargetSource = NAME_None;
					}
				}
			}

			Skeleton->Modify();
			// delete now
			Skeleton->AnimRetargetSources.Remove(SourceName);
			Skeleton->CallbackRetargetSourceChanged();
		}
	}
}

void FEditableSkeleton::RefreshRetargetSources(const TArray<FName>& InRetargetSourceNames)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("RetargetSourceWindow_Refresh", "Refresh Retarget Source"));
	for (const FName& RetargetSourceName : InRetargetSourceNames)
	{
		Skeleton->Modify();

		// refresh pose now
		Skeleton->UpdateRetargetSource(RetargetSourceName);

		// feedback of pose has been updated
		FFormatNamedArguments Args;
		Args.Add(TEXT("RetargetSourceName"), FText::FromString(RetargetSourceName.ToString()));
		FNotificationInfo Info(FText::Format(LOCTEXT("RetargetSourceWindow_RefreshPose", "Retarget Source {RetargetSourceName} is refreshed"), Args));
		Info.ExpireDuration = 5.0f;
		Info.bUseLargeFont = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_None);
		}
	}
}

void FEditableSkeleton::AddCompatibleSkeleton(const USkeleton* InCompatibleSkeleton)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("AddedCompatibleSkeletons", "Add Compatible Skeleton"));
	Skeleton->Modify();

	Skeleton->AddCompatibleSkeleton(InCompatibleSkeleton);

	// Inform asset families
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaModule.BroadcastAssetFamilyChange();
}

void FEditableSkeleton::RemoveCompatibleSkeleton(const USkeleton* InCompatibleSkeleton)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("RemoveCompatibleSkeletons", "Remove Compatible Skeleton"));
	Skeleton->Modify();

	Skeleton->RemoveCompatibleSkeleton(InCompatibleSkeleton);

	// Inform asset families
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaModule.BroadcastAssetFamilyChange();
}

void FEditableSkeleton::RemoveUnusedBones()
{
	if (!Skeleton)
	{
		return;
	}
	TArray<FName> SkeletonBones;
	FindUnusedBones(SkeletonBones);

	if (SkeletonBones.Num() > 0)
	{
		const FText RemoveBoneMessage = FText::Format(LOCTEXT("RemoveBoneWarning", "Continuing will remove the following bones from the skeleton '{0}'. These bones are not being used by any of the SkeletalMeshes assigned to this skeleton.\n\nOnce the bones have been removed, all loaded animations for this skeleton will be recompressed (any that aren't loaded will be recompressed the next time they are loaded)."), FText::FromString(Skeleton->GetName()));

		// Ask User whether they would like to remove the bones from the skeleton
		if (SSkeletonBoneRemoval::ShowModal(SkeletonBones, RemoveBoneMessage))
		{
			//Remove these bones from the skeleton
			Skeleton->RemoveBonesFromSkeleton(SkeletonBones, true);
		}
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoBonesToRemove", "No unused bones were found."));
	}

	OnTreeRefresh.Broadcast();
}

void FEditableSkeleton::FindUnusedBones(TArray<FName>& OutSkeletonBones, bool bSilent /*=false*/)
{
	if (!Skeleton)
	{
		OutSkeletonBones.Empty();
		return;
	}
	// This operation can take a long time, so unless we're running silently, give the user a heads-up
	if (!bSilent)
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());

		FString SkeletonString = FAssetData(Skeleton).GetExportTextName();
		Filter.TagsAndValues.Add(USkeletalMesh::GetSkeletonMemberName(), SkeletonString);

		TArray<FAssetData> SkeletalMeshes;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().GetAssets(Filter, SkeletalMeshes);

		FText TimeTakenMessage = FText::Format(LOCTEXT("TimeTakenWarning", "In order to verify bone use, all Skeletal Meshes that use this skeleton will be loaded.  This may take some time.\n\nProceed?\n\nNumber of Meshes: {0}"), FText::AsNumber(SkeletalMeshes.Num()));

		if (FMessageDialog::Open(EAppMsgType::YesNo, TimeTakenMessage) != EAppReturnType::Yes)
		{
			return;
		}
	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	
	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
	{
		OutSkeletonBones.Add(RefSkeleton.GetBoneName(BoneIndex));
	}
	
	FARFilter Filter;
	Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	
	FString SkeletonString = FAssetData(Skeleton).GetExportTextName();
	Filter.TagsAndValues.Add(USkeletalMesh::GetSkeletonMemberName(), SkeletonString);
	
	TArray<FAssetData> SkeletalMeshes;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssets(Filter, SkeletalMeshes);
	
	const FText StatusUpdate = FText::Format(LOCTEXT("RemoveUnusedBones_ProcessingAssetsFor", "Processing Skeletal Meshes for {0}"), FText::FromString(Skeleton->GetName()));
	GWarn->BeginSlowTask(StatusUpdate, true);
	
	// Loop through virtual bones and remove the bones they use from the list
	for (const FVirtualBone& VB : Skeleton->GetVirtualBones())
	{
		OutSkeletonBones.Remove(VB.SourceBoneName);
		OutSkeletonBones.Remove(VB.TargetBoneName);
		if (OutSkeletonBones.Num() == 0)
		{
			break;
		}
	}
	
	if (OutSkeletonBones.Num() != 0)
	{
		// Loop through all SkeletalMeshes and remove the bones they use from our list
		for (int32 MeshIdx = 0; MeshIdx < SkeletalMeshes.Num(); ++MeshIdx)
		{
			GWarn->StatusUpdate(MeshIdx, SkeletalMeshes.Num(), StatusUpdate);
	
			USkeletalMesh* Mesh = Cast<USkeletalMesh>(SkeletalMeshes[MeshIdx].GetAsset());
			const FReferenceSkeleton& MeshRefSkeleton = Mesh->GetRefSkeleton();
	
			for (int32 BoneIndex = 0; BoneIndex < MeshRefSkeleton.GetRawBoneNum(); ++BoneIndex)
			{
				OutSkeletonBones.Remove(MeshRefSkeleton.GetBoneName(BoneIndex));
				if (OutSkeletonBones.Num() == 0)
				{
					break;
				}
			}
			if (OutSkeletonBones.Num() == 0)
			{
				break;
			}
		}
	}
	
	GWarn->EndSlowTask();
	
	//Remove bones that are a parent to bones we aren't removing
	for (int32 BoneIndex = RefSkeleton.GetRawBoneNum() - 1; BoneIndex >= 0; --BoneIndex)
	{
		FName CurrBoneName = RefSkeleton.GetBoneName(BoneIndex);
		if (!OutSkeletonBones.Contains(CurrBoneName))
		{
			//We aren't removing this bone, so remove parent from list of bones to remove too
			int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				OutSkeletonBones.Remove(RefSkeleton.GetBoneName(ParentIndex));
			}
		}
	}
}

void FEditableSkeleton::UpdateSkeletonReferencePose(USkeletalMesh* InSkeletalMesh)
{
	if (!Skeleton)
	{
		return;
	}
	Skeleton->UpdateReferencePoseFromMesh(InSkeletalMesh);
}

void FEditableSkeleton::RegisterSlotNode(const FName& InSlotName)
{
	if (!Skeleton)
	{
		return;
	}
	if(!Skeleton->ContainsSlotName(InSlotName))
	{
		const FScopedTransaction Transaction(LOCTEXT("RegisterSlotNode", "Register Slot Node"));
		Skeleton->Modify();

		if(Skeleton->RegisterSlotNode(InSlotName))
		{
			OnSlotsChanged.Broadcast();
		}
	}
}

bool FEditableSkeleton::AddSlotGroupName(const FName& InSlotName)
{
	if (!Skeleton)
	{
		return false;
	}
	const FScopedTransaction Transaction(LOCTEXT("AddSlotGroupName", "Add Slot Group Name"));
	Skeleton->Modify();
	bool bResult = Skeleton->AddSlotGroupName(InSlotName);
	OnSlotsChanged.Broadcast();
	return bResult;
}

void FEditableSkeleton::SetSlotGroupName(const FName& InSlotName, const FName& InGroupName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("SetSlotGroupName", "Set Slot Group Name"));
	Skeleton->Modify();
	Skeleton->SetSlotGroupName(InSlotName, InGroupName);

	OnSlotsChanged.Broadcast();
}

void FEditableSkeleton::DeleteSlotName(const FName& InSlotName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("DeleteSlotName", "Delete Slot Name"));
	Skeleton->Modify();
	Skeleton->RemoveSlotName(InSlotName);

	OnSlotsChanged.Broadcast();
}

void FEditableSkeleton::DeleteSlotGroup(const FName& InGroupName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("DeleteSlotGroup", "Delete Slot Group"));
	Skeleton->Modify();
	Skeleton->RemoveSlotGroup(InGroupName);

	OnSlotsChanged.Broadcast();
}

void FEditableSkeleton::RenameSlotName(const FName InOldSlotName, const FName InNewSlotName)
{
	if (!Skeleton)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("RenameSlotName", "Rename Slot Name"));
	Skeleton->Modify();
	Skeleton->RenameSlotName(InOldSlotName, InNewSlotName);

	OnSlotsChanged.Broadcast();
}


FDelegateHandle FEditableSkeleton::RegisterOnSlotsChanged(const FSimpleMulticastDelegate::FDelegate& InOnSlotsChanged)
{
	return OnSlotsChanged.Add(InOnSlotsChanged);
}

void FEditableSkeleton::UnregisterOnSlotsChanged(FDelegateHandle InHandle)
{
	OnSlotsChanged.Remove(InHandle);
}

#undef LOCTEXT_NAMESPACE
