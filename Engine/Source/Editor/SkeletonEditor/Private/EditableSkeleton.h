// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "BoneContainer.h"
#include "Animation/Skeleton.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "IEditableSkeleton.h"

class IPersonaPreviewScene;
class SBlendProfilePicker;
class SSkeletonTree;
class UBlendProfile;
class URig;
class USkeletalMeshSocket;

/** View-model for a skeleton tree */
class FEditableSkeleton : public IEditableSkeleton, public FGCObject, public TSharedFromThis<FEditableSkeleton>
{
public:
	/** String used as a header for text based copy-paste of sockets */
	static const FString SocketCopyPasteHeader;

public:
	FEditableSkeleton(USkeleton* InSkeleton);

	/** IEditableSkeleton interface */
	virtual const USkeleton& GetSkeleton() const override;
	virtual bool IsSkeletonValid() const override;
	virtual const TArray<class UBlendProfile*>& GetBlendProfiles() const override;
	virtual class UBlendProfile* GetBlendProfile(const FName& InBlendProfileName) override;
	virtual class UBlendProfile* CreateNewBlendProfile(const FName& InBlendProfileName) override;
	virtual void RemoveBlendProfile(UBlendProfile* InBlendProfile) override;
	virtual UBlendProfile* RenameBlendProfile(const FName& InBlendProfileName, const FName& InNewBlendProfileName) override;
	virtual void SetBlendProfileScale(const FName& InBlendProfileName, const FName& InBoneName, float InNewScale, bool bInRecurse) override;
	virtual void SetBlendProfileMode(FName InBlendProfileName, EBlendProfileMode ProfileMode);
	virtual USkeletalMeshSocket* DuplicateSocket(const FSelectedSocketInfo& SocketInfoToDuplicate, const FName& NewParentBoneName, USkeletalMesh* InSkeletalMesh) override;
	virtual int32 ValidatePreviewAttachedObjects() override;
	virtual int32 DeleteAnimNotifies(const TArray<FName>& InSelectedNotifyNames, bool bDeleteFromAnimations = true) override;
	virtual int32 DeleteSyncMarkers(const TArray<FName>& ISyncMarkerNames, bool bDeleteFromAnimations = true) override;
	virtual void AddNotify(FName NewName) override;
	virtual void AddSyncMarker(FName NewName) override;
	virtual int32 RenameNotify(const FName NewName, const FName OldName, bool bRenameInAnimations = true) override;
	virtual int32 RenameSyncMarker(const FName NewName, const FName OldName, bool bRenameInAnimations = true) override;
	virtual void BroadcastNotifyChanged() override;
	virtual void GetCompatibleAnimSequences(TArray<struct FAssetData>& OutAssets) override;
	virtual void RenameSocket(const FName OldSocketName, const FName NewSocketName, USkeletalMesh* InSkeletalMesh) override;
	virtual void SetSocketParent(const FName& SocketName, const FName& NewParentName, USkeletalMesh* InSkeletalMesh, bool bKeepAbsoluteLocation) override;
	virtual bool DoesSocketAlreadyExist(const class USkeletalMeshSocket* InSocket, const FText& InSocketName, ESocketParentType SocketParentType, USkeletalMesh* InSkeletalMesh) const override;
	virtual bool DoesVirtualBoneAlreadyExist(const FString& InVBName) const override;
	virtual void RenameVirtualBone(const FName OriginalName, const FName InVBName) override;
	virtual void SetPreviewMesh(class USkeletalMesh* InSkeletalMesh) override;
	virtual void LoadAdditionalPreviewSkeletalMeshes() override;
	virtual void SetAdditionalPreviewSkeletalMeshes(class UDataAsset* InPreviewCollectionAsset) override;
	virtual void RenameRetargetSource(const FName InOldName, const FName InNewName) override;
	virtual void AddRetargetSource(const FName& InName, USkeletalMesh* InReferenceMesh) override;
	virtual void DeleteRetargetSources(const TArray<FName>& InRetargetSourceNames) override;
	virtual void RefreshRetargetSources(const TArray<FName>& InRetargetSourceNames) override;
	virtual void AddCompatibleSkeleton(const USkeleton* InCompatibleSkeleton) override;
	virtual void RemoveCompatibleSkeleton(const USkeleton* InCompatibleSkeleton) override;
	virtual void RemoveUnusedBones() override;
	virtual void FindUnusedBones(TArray<FName>& OutUnusedBones, bool bSilent = false) override;
	virtual void UpdateSkeletonReferencePose(USkeletalMesh* InSkeletalMesh) override;
	virtual void RegisterSlotNode(const FName& InSlotName) override;
	virtual bool AddSlotGroupName(const FName& InSlotName) override;
	virtual void SetSlotGroupName(const FName& InSlotName, const FName& InGroupName) override;
	virtual void DeleteSlotName(const FName& InSlotName) override;
	virtual void DeleteSlotGroup(const FName& InGroupName) override;
	virtual void RenameSlotName(const FName InOldSlotName, const FName InNewSlotName) override;
	virtual void RegisterOnNotifiesChanged(const FSimpleMulticastDelegate::FDelegate& InDelegate) override;
	virtual void UnregisterOnNotifiesChanged(FDelegateUserObject Thing) override;
	virtual FDelegateHandle RegisterOnSlotsChanged(const FSimpleMulticastDelegate::FDelegate& InOnSlotsChanged) override;
	virtual void UnregisterOnSlotsChanged(FDelegateHandle InHandle) override;
	virtual void SetBoneTranslationRetargetingMode(FName InBoneName, EBoneTranslationRetargetingMode::Type NewRetargetingMode) override;
	virtual EBoneTranslationRetargetingMode::Type GetBoneTranslationRetargetingMode(FName InBoneName) const override;
	virtual void RefreshBoneTree() override;
	virtual USkeletalMeshSocket* AddSocket(const FName& InBoneName) override;

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FEditableSkeleton");
	}

	/** Generates a unique socket name from the input name, by changing the FName's number */
	FName GenerateUniqueSocketName(FName InName, USkeletalMesh* InSkeletalMesh);

	/** Handle the user pasting sockets */
	void HandlePasteSockets(const FName& InBoneName, USkeletalMesh* InSkeletalMesh);

	/** Handles adding a socket to the specified bone (i.e. skeleton, not mesh) */
	USkeletalMeshSocket* HandleAddSocket(const FName& InBoneName);

	/** Handle adding a new virtual bone to the skeleton */
	bool HandleAddVirtualBone(const FName SourceBoneName, const FName TargetBoneName);

	/** Handle adding a new virtual bone to the skeleton */
	bool HandleAddVirtualBone(const FName SourceBoneName, const FName TargetBoneName, FName& NewVirtualBoneName);

	/** Function to customize a socket - this essentially copies a socket from the skeleton to the mesh */
	void HandleCustomizeSocket(USkeletalMeshSocket* InSocketToCustomize, USkeletalMesh* InSkeletalMesh);

	/** Function to promote a socket - this essentially copies a socket from the mesh to the skeleton */
	void HandlePromoteSocket(USkeletalMeshSocket* InSocketToPromote);

	/** Handle removing all attached assets, optionally keeping a preview scene in sync */
	void HandleRemoveAllAssets(TSharedPtr<class IPersonaPreviewScene> InPreviewScene);

	/** Handle attaching assets to the skeleton or mesh, optionally keeping a preview scene in sync */
	void HandleAttachAssets(const TArray<UObject*>& InObjects, const FName& InAttachToName, bool bAttachToMesh, TSharedPtr<class IPersonaPreviewScene> InPreviewScene);

	/** Handle deleting attached assets, optionally keeping a preview scene in sync */
	void HandleDeleteAttachedAssets(const TArray<FPreviewAttachedObjectPair>& InAttachedObjects, TSharedPtr<class IPersonaPreviewScene> InPreviewScene);

	/** Handle deleting sockets, optionally keeping a preview scene in sync */
	void HandleDeleteSockets(const TArray<FSelectedSocketInfo>& InSocketInfo, TSharedPtr<class IPersonaPreviewScene> InPreviewScene);

	/** Handle deleting virtual bones, optionally keeping a preview scene in sync */
	void HandleDeleteVirtualBones(const TArray<FName>& InVirtualBoneInfo, TSharedPtr<class IPersonaPreviewScene> InPreviewScene);

	/** Set Bone Translation Retargeting Mode for the passed-in bones and their children. */
	void SetBoneTranslationRetargetingModeRecursive(const TArray<FName>& InBoneNames, EBoneTranslationRetargetingMode::Type NewRetargetingMode);

	/** Sets the blend scale for the selected bones and all of their children */
	void RecursiveSetBlendProfileScales(const FName& InBlendProfileName, const TArray<FName>& InBoneNames, float InScaleToSet);

	/** Create a new skeleton tree to edit this editable skeleton */
	TSharedRef<class ISkeletonTree> CreateSkeletonTree(const struct FSkeletonTreeArgs& InSkeletonTreeArgs);

	/** Create a new blend profile picker to edit this editable skeleton's blend profiles */
	TSharedRef<class SWidget> CreateBlendProfilePicker(const struct FBlendProfilePickerArgs& InArgs);

	/** Check whether we have any widgets editing our data */
	bool IsEdited() const { return SkeletonTrees.Num() > 0 || BlendProfilePickers.Num() > 0; }

	/** Register for skeleton changes */
	void RegisterOnSkeletonHierarchyChanged(const USkeleton::FOnSkeletonHierarchyChanged& InDelegate);

	/** Unregister for skeleton changes */
	void UnregisterOnSkeletonHierarchyChanged(FDelegateUserObject Thing);

	/** Wrap USkeleton::RecreateBoneTree */
	void RecreateBoneTree(USkeletalMesh* NewPreviewMesh);

private:
	/** Helper function for deleting attached objects */
	void DeleteAttachedObjects(FPreviewAssetAttachContainer& AttachedAssets, TSharedPtr<class IPersonaPreviewScene> InPreviewScene);

	/** Helper function for finding animations that use certain curves */
	void GetAssetsContainingCurves(const FName& InContainerName, const TArray<FName>& InNames, TArray<FAssetData>& OutAssets) const;

private:
	/** The skeleton we are editing */
	TObjectPtr<class USkeleton> Skeleton;

	/** All skeleton tree widgets that are editing this skeleton */
	TArray<TWeakPtr<class SSkeletonTree>> SkeletonTrees;

	/** All blend profile widgets that are editing this skeleton */
	TArray<TWeakPtr<class SBlendProfilePicker>> BlendProfilePickers;

	/** Delegate called when trees need refreshing */
	FSimpleMulticastDelegate OnTreeRefresh;

	/** Delegate called when notifies are modified */
	FSimpleMulticastDelegate OnNotifiesChanged;

	/** Delegate called when slots are modified */
	FSimpleMulticastDelegate OnSlotsChanged;
};
