// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlendProfilePicker.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorEngine.h"
#include "EngineGlobals.h"
#include "Animation/BlendProfile.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "BlendProfilePicker"

namespace BlendProfilePickerNames
{
	static const FText BlendProfileModeName = LOCTEXT("Blend Profile Mode Name", "Blend Profile");
	static const FText BlendMaskModeName = LOCTEXT("Blend Mask Mode Name", "Blend Mask");

	const FText& GetNameForMode(EBlendProfileMode InMode)
	{
		return (InMode == EBlendProfileMode::BlendMask) ? BlendProfilePickerNames::BlendMaskModeName : BlendProfilePickerNames::BlendProfileModeName;
	}
}

class SBlendProfileMenuEntry : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FBlendProfileModeChanged, EBlendProfileMode);

	SLATE_BEGIN_ARGS(SBlendProfileMenuEntry){}
		SLATE_ARGUMENT( FText, LabelOverride )
		/** Called to when an entry is clicked */
		SLATE_EVENT( FExecuteAction, OnOpenClickedDelegate )
		/** Called to when the button remove an entry is clicked */
		SLATE_EVENT( FExecuteAction, OnRemoveClickedDelegate )
		SLATE_EVENT(FBlendProfileModeChanged, OnProfileModeChangedDelegate)
		/** Whether to show the remove button */
		SLATE_ARGUMENT(bool, AllowModify)
		SLATE_ARGUMENT(TWeakObjectPtr<UBlendProfile>, BlendProfile)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		const FText DisplayName = InArgs._LabelOverride;
		OnOpenClickedDelegate = InArgs._OnOpenClickedDelegate;
		OnRemoveClickedDelegate = InArgs._OnRemoveClickedDelegate;
		OnProfileModeChangedDelegate = InArgs._OnProfileModeChangedDelegate;
		BlendProfile = InArgs._BlendProfile;

		FSlateFontInfo MenuEntryFont = FAppStyle::GetFontStyle( "Menu.Label.Font" );

		TSharedPtr<SHorizontalBox> HorizontalBox;

		EBlendProfileMode BlendProfileMode = GetProfileMode();

		ChildSlot
		[
			SNew(SButton)
			.ButtonStyle( FAppStyle::Get(), "Menu.Button" )
			.ForegroundColor( TAttribute<FSlateColor>::Create( TAttribute<FSlateColor>::FGetter::CreateRaw( this, &SBlendProfileMenuEntry::InvertOnHover ) ) )
			.ToolTipText(LOCTEXT("OpenBlendProfileToolTip", "Select this profile for editing."))
			.OnClicked(this, &SBlendProfileMenuEntry::OnOpen)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.ContentPadding( FMargin(4.0, 2.0) )
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(HorizontalBox, SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding( FMargin( 12.0, 0.0 ) )
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Fill)
					[
						SNew(STextBlock)
						.Font(MenuEntryFont)
						.ColorAndOpacity(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateRaw(this, &SBlendProfileMenuEntry::InvertOnHover)))
						.Text(DisplayName)
					]
				]
			]
		];

		if(InArgs._AllowModify)
		{
			HorizontalBox->AddSlot()
				.Padding(FMargin(8.0, 0.0))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ButtonColorAndOpacity(FLinearColor::Transparent)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.HasDownArrow(false)
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.ContentPadding(2.0f)
					.ToolTipText(FText::Format(LOCTEXT("ModifyBlendProfileToolTipFmt", "Modify {0}"), DisplayName))
					.MenuPlacement(MenuPlacement_MenuRight)
					.OnGetMenuContent(this, &SBlendProfileMenuEntry::GetMenuContent)
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]
				];
		}
	}

	void OnProfileModeSelected(EBlendProfileMode InMode)
	{
		OnProfileModeChangedDelegate.ExecuteIfBound(InMode);
	}

	EBlendProfileMode GetProfileMode()
	{
		if (BlendProfile.IsValid())
		{
			return BlendProfile->Mode;
		}
		return EBlendProfileMode::WeightFactor;
	}

	bool IsProfileModeSelected(EBlendProfileMode InMode)
	{
		if (BlendProfile.IsValid())
		{
			return BlendProfile->Mode == InMode;
		}
		return false;
	}

	TSharedRef<SWidget> GetMenuContent()
	{
		FMenuBuilder MenuBuilder(false, nullptr, TSharedPtr<FExtender>(), true);

		EBlendProfileMode BlendProfileMode = GetProfileMode();

		if (BlendProfileMode != EBlendProfileMode::BlendMask)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("BlendProfileMode", "Mode"));
			{
				UEnum* ModeEnum = StaticEnum<EBlendProfileMode>();
				check(ModeEnum);

				// Last enum entry is _MAX
				int32 NumEnums = ModeEnum->NumEnums() - 1;
				static const TCHAR* HiddenMeta = TEXT("Hidden");

				for (int32 EnumIndex = 0; EnumIndex < NumEnums; ++EnumIndex)
				{
					if (!ModeEnum->HasMetaData(HiddenMeta, EnumIndex))
					{
						EBlendProfileMode IndexMode = (EBlendProfileMode)ModeEnum->GetValueByIndex(EnumIndex);
						MenuBuilder.AddMenuEntry(
							ModeEnum->GetDisplayNameTextByIndex(EnumIndex),
							ModeEnum->GetToolTipTextByIndex(EnumIndex),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(this, &SBlendProfileMenuEntry::OnProfileModeSelected, IndexMode),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(this, &SBlendProfileMenuEntry::IsProfileModeSelected, IndexMode)
							),
							NAME_None,
							EUserInterfaceActionType::RadioButton);
					}
				}
			}
			MenuBuilder.EndSection();
		}

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("BlendProfileActions", "Actions"));
		{
			const FText& ModeName = BlendProfilePickerNames::GetNameForMode(BlendProfileMode);
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("RemoveBlendProfile", "Remove {0}"), ModeName),
				FText::Format(LOCTEXT("RemoveBlendProfile_ToolTip", "Remove this {0} from the skeleton"), ModeName),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]()
					{
						OnRemoveClickedDelegate.ExecuteIfBound();
						FSlateApplication::Get().DismissAllMenus();
					})));

		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	FReply OnOpen()
	{
		OnOpenClickedDelegate.ExecuteIfBound();
		FSlateApplication::Get().DismissAllMenus();
		return FReply::Handled();
	}

private:
	FSlateColor InvertOnHover() const
	{
		if ( this->IsHovered() )
		{
			return FLinearColor::Black;
		}
		else
		{
			return FSlateColor::UseForeground();
		}
	}

	FExecuteAction OnOpenClickedDelegate;
	FExecuteAction OnRemoveClickedDelegate;
	FBlendProfileModeChanged OnProfileModeChangedDelegate;
	TWeakObjectPtr<UBlendProfile> BlendProfile;
};

//////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBlendProfilePicker::Construct(const FArguments& InArgs, TSharedRef<class IEditableSkeleton> InEditableSkeleton)
{
	bShowNewOption = InArgs._AllowNew;
	bShowClearOption = InArgs._AllowClear;
	bIsStandalone = InArgs._Standalone;
	EditableSkeleton = InEditableSkeleton;
	SupportedBlendProfileModes = InArgs._SupportedBlendProfileModes;
	bAllowModify = InArgs._AllowModify;

	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}

	PropertyHandle = InArgs._PropertyHandle;

	if (PropertyHandle.IsValid())
	{
		FSimpleDelegate OnModeChanged = FSimpleDelegate::CreateLambda([this]
			{
				UObject * PropertyValue = nullptr;
				PropertyHandle->GetValue(PropertyValue);
				UBlendProfile * CurrentProfile = Cast<UBlendProfile>(PropertyValue);
				// Avoid broadcasting to not double call FBlendProfileCustomization::OnBlendProfileChanged
				SetSelectedProfile(CurrentProfile, false);
			});

		// Sometimes, the property value changes externally (not through the blend profile picker i.e. Reset to default). Must notify the blend profile picker
		PropertyHandle->SetOnPropertyValueChanged(OnModeChanged);
	}

	if(InArgs._InitialProfile != nullptr && InEditableSkeleton->GetBlendProfiles().Contains(InArgs._InitialProfile))
	{
		SelectedProfileName = InArgs._InitialProfile->GetFName();
	}
	else
	{
		SelectedProfileName = NAME_None;
	}

	BlendProfileSelectedDelegate = InArgs._OnBlendProfileSelected;

	TSharedRef<SWidget> TextBlock = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		.Text(this, &SBlendProfilePicker::GetSelectedProfileName);

	TSharedPtr<SWidget> ButtonContent;

	if (bIsStandalone)
	{
		ButtonContent = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("SkeletonTree.BlendProfile"))
			]
			+SHorizontalBox::Slot()
			.Padding(2.0f, 0, 8.0f, 0)
			.VAlign(VAlign_Center)
			[
				TextBlock
			];
	}
	else
	{
		ButtonContent = TextBlock;
	}

	ChildSlot
	[
		SNew(SComboButton)
		.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
		.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
		.ContentPadding(2.0f)
		.OnGetMenuContent(this, &SBlendProfilePicker::GetMenuContent)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SBlendProfilePicker::~SBlendProfilePicker()
{
	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->UnregisterForUndo(this);
	}
}

FText SBlendProfilePicker::GetSelectedProfileName() const
{
	UBlendProfile* SelectedProfile = EditableSkeleton->GetBlendProfile(SelectedProfileName);
	if(SelectedProfile)
	{
		if (bIsStandalone)
		{
			return FText::Format(FText(LOCTEXT("SelectedNameEntryStandalone", "{0}: {1}")), BlendProfilePickerNames::GetNameForMode(SelectedProfile->Mode), FText::FromName(SelectedProfileName));
		}
		else
		{
			return FText::Format(FText(LOCTEXT("SelectedNameEntry", "{0}")), FText::FromName(SelectedProfileName));
		}
	}
	if (bIsStandalone)
	{
		return FText(LOCTEXT("NoSelectionEntryStandalone", "Blend Profile/Mask: None"));
	}
	else
	{
		return FText(LOCTEXT("NoSelectionEntry", "None"));
	}
}

TSharedRef<SWidget> SBlendProfilePicker::GetMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	const bool bHasSettingsSection = bShowNewOption || bShowClearOption;

	if(bHasSettingsSection)
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("MenuSettings", "Settings"));
		{
			if(bShowNewOption)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("CreateNewBlendProfile", "Create New Blend Profile"),
					LOCTEXT("CreateNewBlendProfile_ToolTip", "Creates a new blend profile inside the skeleton."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SBlendProfilePicker::OnCreateNewProfile, EBlendProfileMode::TimeFactor))); //Default blend profiles to time factor

				MenuBuilder.AddMenuEntry(
					LOCTEXT("CreateNewBlendMask", "Create New Blend Mask"),
					LOCTEXT("CreateNewBlendMask_ToolTip", "Creates a new blend mask inside the skeleton."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SBlendProfilePicker::OnCreateNewProfile, EBlendProfileMode::BlendMask)));
			}

			if(bShowClearOption)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Clear", "Clear"),
					LOCTEXT("Clear_ToolTip", "Clear the selected blend profile."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SBlendProfilePicker::OnClearSelection)));
			}
		}
		MenuBuilder.EndSection();
	}

	const static FText BlendProfileHeaders[] =
	{
		LOCTEXT("BlendProfilesTimeBased", "Blend Profiles - Time"),
		LOCTEXT("BlendProfilesWeightBased", "Blend Profiles - Weight"),
		LOCTEXT("BlendMasks", "Blend Masks")
	};

	UEnum* ModeEnum = StaticEnum<EBlendProfileMode>();
	check(ModeEnum);
	// Last enum entry is _MAX
	int32 NumEnums = ModeEnum->NumEnums() - 1;

	TArray<TArray<UBlendProfile*>> BlendProfilesFiltered;
	BlendProfilesFiltered.SetNum(NumEnums);

	// Build a filtered profile list by mode
	for (UBlendProfile* Profile : EditableSkeleton->GetBlendProfiles())
	{
		if (Profile)
		{
			BlendProfilesFiltered[ModeEnum->GetIndexByValue((int64)Profile->GetMode())].Add(Profile);
		}
	}

	const bool bSupportsBlendMasks = EnumHasAnyFlags(SupportedBlendProfileModes, EBlendProfilePickerMode::BlendMask);
	const bool bSupportsBlendProfiles = EnumHasAnyFlags(SupportedBlendProfileModes, EBlendProfilePickerMode::BlendProfile);
	for (int32 ModeIndex = 0; ModeIndex < NumEnums; ++ModeIndex)
	{
		EBlendProfileMode IndexMode = (EBlendProfileMode)ModeEnum->GetValueByIndex(ModeIndex);
		if ((IndexMode == EBlendProfileMode::BlendMask && bSupportsBlendMasks)
			|| (IndexMode != EBlendProfileMode::BlendMask && bSupportsBlendProfiles))
		{
			// Note: Section won't get populated if there are no available items for this mode type
			MenuBuilder.BeginSection(NAME_None, BlendProfileHeaders[ModeIndex]);
			for (UBlendProfile* Profile : BlendProfilesFiltered[ModeIndex])
			{
				if (IndexMode == Profile->GetMode())
				{
					MenuBuilder.AddWidget(
						SNew(SBlendProfileMenuEntry)
						.LabelOverride(FText::FromString(Profile->GetName()))
						.OnOpenClickedDelegate(FExecuteAction::CreateSP(this, &SBlendProfilePicker::OnProfileSelected, Profile->GetFName()))
						.OnRemoveClickedDelegate(FExecuteAction::CreateSP(this, &SBlendProfilePicker::OnProfileRemoved, Profile->GetFName()))
						.OnProfileModeChangedDelegate(SBlendProfileMenuEntry::FBlendProfileModeChanged::CreateSP(this, &SBlendProfilePicker::OnProfileModeChanged, Profile->GetFName()))
						.BlendProfile(MakeWeakObjectPtr(Profile))
						.AllowModify(bAllowModify),
						FText(),
						true
					);
				}
			}
		}

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SBlendProfilePicker::OnProfileSelected(FName InBlendProfileName)
{
	SelectedProfileName = InBlendProfileName;
	BlendProfileSelectedDelegate.ExecuteIfBound(EditableSkeleton->GetBlendProfile(SelectedProfileName));
}

void SBlendProfilePicker::OnClearSelection()
{
	SelectedProfileName = NAME_None;
	BlendProfileSelectedDelegate.ExecuteIfBound(nullptr);
}

void SBlendProfilePicker::OnProfileRemoved(FName InBlendProfileName)
{
	EditableSkeleton->RemoveBlendProfile(EditableSkeleton->GetBlendProfile(InBlendProfileName));
	SelectedProfileName = NAME_None;
	BlendProfileSelectedDelegate.ExecuteIfBound(nullptr);
}

void SBlendProfilePicker::OnProfileModeChanged(EBlendProfileMode ProfileMode, FName InBlendProfileName)
{
	EditableSkeleton->SetBlendProfileMode(InBlendProfileName, ProfileMode);
}

void SBlendProfilePicker::OnCreateNewProfile(EBlendProfileMode InMode)
{
	TSharedRef<STextEntryPopup> TextEntry = SNew(STextEntryPopup)
		.Label(LOCTEXT("NewProfileName", "Profile Name"))
		.OnTextCommitted(this, &SBlendProfilePicker::OnCreateNewProfileComitted, InMode)
		.OnVerifyTextChanged_Lambda([](const FText& InNewText, FText& OutErrorMessage) -> bool
		{
			return FName::IsValidXName(InNewText.ToString(), INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage);
		});

	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup));
}

void SBlendProfilePicker::OnCreateNewProfileComitted(const FText& NewName, ETextCommit::Type CommitType, EBlendProfileMode InMode)
{
	FSlateApplication::Get().DismissAllMenus();

	if((CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus) && EditableSkeleton.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("Trans_NewProfile", "Create new blend profile."));

		FName NameToUse = FName(*NewName.ToString());

		// Only create if we don't have a matching profile
		if(UBlendProfile* FoundProfile = EditableSkeleton->GetBlendProfile(NameToUse))
		{
			OnProfileSelected(FoundProfile->GetFName());
		}
		else if(UBlendProfile* NewProfile = EditableSkeleton->CreateNewBlendProfile(NameToUse))
		{
			// Set our initial blend profile mode. Blend masks can't change this.
			NewProfile->Mode = InMode;
			OnProfileSelected(NewProfile->GetFName());
		}
	}
}

void SBlendProfilePicker::SetSelectedProfile(UBlendProfile* InProfile, bool bBroadcast /*= true*/)
{
	if(EditableSkeleton->GetBlendProfiles().Contains(InProfile))
	{
		SelectedProfileName = InProfile->GetFName();
		if(bBroadcast)
		{
			BlendProfileSelectedDelegate.ExecuteIfBound(InProfile);
		}
	}
	else if(!InProfile)
	{
		OnClearSelection();
	}
}

UBlendProfile* const SBlendProfilePicker::GetSelectedBlendProfile() const
{
	return EditableSkeleton->GetBlendProfile(SelectedProfileName);
}

FName SBlendProfilePicker::GetSelectedBlendProfileName() const
{
	UBlendProfile* SelectedProfile = EditableSkeleton->GetBlendProfile(SelectedProfileName);

	return SelectedProfile ? SelectedProfile->GetFName() : NAME_None;
}

void SBlendProfilePicker::PostUndo(bool bSuccess)
{
	BlendProfileSelectedDelegate.ExecuteIfBound(EditableSkeleton->GetBlendProfile(SelectedProfileName));
}

void SBlendProfilePicker::PostRedo(bool bSuccess)
{
	BlendProfileSelectedDelegate.ExecuteIfBound(EditableSkeleton->GetBlendProfile(SelectedProfileName));
}

#undef LOCTEXT_NAMESPACE
