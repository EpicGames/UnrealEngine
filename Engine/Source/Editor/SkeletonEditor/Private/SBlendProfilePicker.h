// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IEditableSkeleton.h"
#include "Widgets/SCompoundWidget.h"
#include "EditorUndoClient.h"
#include "BlendProfilePicker.h"

class UBlendProfile;
enum class EBlendProfileMode : uint8;

// Picker for UBlendProfile instances inside a USkeleton
class SBlendProfilePicker : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SBlendProfilePicker)
		: _InitialProfile(nullptr)
		, _OnBlendProfileSelected()
		, _SupportedBlendProfileModes(EBlendProfilePickerMode::AllModes)
		, _AllowNew(true)
		, _AllowClear(true)
		, _AllowModify(true)
		, _Standalone(false)
	{}
		// Initial blend profile selected
		SLATE_ARGUMENT(UBlendProfile*, InitialProfile)
		// Delegate to call when the picker selection is changed
		SLATE_EVENT(FOnBlendProfileSelected, OnBlendProfileSelected)
		// Only display Blend Profiles w/ specified Blend Profile modes (EBlendProfilePickerMode values are flags.)
		SLATE_ARGUMENT(EBlendProfilePickerMode, SupportedBlendProfileModes)
		// Allow the option to create new profiles in the picker
		SLATE_ARGUMENT(bool, AllowNew)
		// Allow the option to clear the profile selection
		SLATE_ARGUMENT(bool, AllowClear)
		// Allow the option to modify (remove/edit settings) the profile from the skeleton
		SLATE_ARGUMENT(bool, AllowModify)
		// Is this a standalone blend profile picker?
		SLATE_ARGUMENT(bool, Standalone)
		// Optional property handle using this widget
		SLATE_ARGUMENT(TSharedPtr<class IPropertyHandle>, PropertyHandle)
	SLATE_END_ARGS()

	~SBlendProfilePicker();

	void Construct(const FArguments& InArgs, TSharedRef<class IEditableSkeleton> InEditableSkeleton);

	/** Set the selected profile externally 
	 *  @param InProfile New Profile to set
	 *  @param bBroadcast Whether or not to broadcast this selection
	 */
	void SetSelectedProfile(UBlendProfile* InProfile, bool bBroadcast = true);

	/** Get the currently selected blend profile */
	UBlendProfile* const GetSelectedBlendProfile() const;

	/** Get the currently selected blend profile name */
	FName GetSelectedBlendProfileName() const;

	/** Create a New Blend profile withe the provided profile name and mode*/
	void OnCreateNewProfileComitted(const FText& NewName, ETextCommit::Type CommitType, EBlendProfileMode InMode);
	
	/* Deselect the current blend profile */
	void OnClearSelection();

private:

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	void OnCreateNewProfile(EBlendProfileMode InMode);

	void OnProfileSelected(FName InBlendProfileName);
	void OnProfileRemoved(FName InBlendProfileName);
	void OnProfileModeChanged(EBlendProfileMode ProfileMode, FName InBlendProfileName);

	FText GetSelectedProfileName() const;

	TSharedRef<SWidget> GetMenuContent();

	bool bShowNewOption;
	bool bShowClearOption;
	bool bIsStandalone;
	bool bAllowModify;
	bool bAllowBlendMask;
	bool bAllowOnlyBlendMask;

	/** Only display Blend Profiles w/ specified Blend Profile modes (EBlendProfilePickerMode values are flags.) */
	EBlendProfilePickerMode SupportedBlendProfileModes;

	FName SelectedProfileName;

	TSharedPtr<class IEditableSkeleton> EditableSkeleton;

	TSharedPtr<class IPropertyHandle> PropertyHandle;

	FOnBlendProfileSelected BlendProfileSelectedDelegate;

};
