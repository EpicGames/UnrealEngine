// Copyright Epic Games, Inc. All Rights Reserved.


#include "SSkeletonTree.h"
#include "Misc/MessageDialog.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/PackageReload.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "UICommandList_Pinnable.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Styling/AppStyle.h"
#include "ActorFactories/ActorFactory.h"
#include "Exporters/Exporter.h"
#include "Sound/SoundBase.h"
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#include "ScopedTransaction.h"
#include "BoneDragDropOp.h"
#include "SocketDragDropOp.h"
#include "SkeletonTreeCommands.h"
#include "Styling/SlateIconFinder.h"
#include "DragAndDrop/AssetDragDropOp.h"

#include "AssetSelection.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ComponentAssetBroker.h"

#include "AnimPreviewInstance.h"

#include "MeshUtilities.h"
#include "UnrealExporter.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/GenericCommands.h"
#include "Animation/BlendProfile.h"
#include "SBlendProfilePicker.h"
#include "IPersonaPreviewScene.h"
#include "IDocumentation.h"
#include "PersonaUtils.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "SkeletonTreeBoneItem.h"
#include "SkeletonTreeSocketItem.h"
#include "SkeletonTreeAttachedAssetItem.h"
#include "SkeletonTreeVirtualBoneItem.h"

#include "BoneSelectionWidget.h"
#include "SkeletonTreeSelection.h"
#include "Widgets/Layout/SGridPanel.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Views/STreeView.h"
#include "IPinnedCommandList.h"
#include "MeshBoneReduction.h"
#include "PersonaModule.h"
#include "SPositiveActionButton.h"
#include "ToolMenus.h"
#include "ToolMenuMisc.h"
#include "SkeletonTreeMenuContext.h"

#define LOCTEXT_NAMESPACE "SSkeletonTree"

const FName	ISkeletonTree::Columns::Name("Name");
const FName	ISkeletonTree::Columns::Retargeting("Retargeting");
const FName ISkeletonTree::Columns::BlendProfile("BlendProfile");
const FName ISkeletonTree::Columns::DebugVisualization("DebugVisualization");

// This is mostly duplicated from SListView, to allow for us to avoid selecting collapsed items
template <typename ItemType>
class SSkeletonTreeView : public STreeView<ItemType>
{
public:

	bool Private_CanItemBeSelected(ItemType InItem) const
	{
		return !(InItem->GetFilterResult() == ESkeletonTreeFilterResult::ShownDescendant && GetMutableDefault<UPersonaOptions>()->bHideParentsWhenFiltering);
	}

	virtual void Private_SelectRangeFromCurrentTo( ItemType InRangeSelectionEnd ) override
	{
		if ( this->GetSelectionMode() == ESelectionMode::None )
		{
			return;
		}

		const TArrayView<const ItemType> ItemsSourceRef = this->SListView<ItemType>::GetItems();

		int32 RangeStartIndex = 0;
		if( TListTypeTraits<ItemType>::IsPtrValid(this->RangeSelectionStart) )
		{
			RangeStartIndex = ItemsSourceRef.Find( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( this->RangeSelectionStart ) );
		}

		int32 RangeEndIndex = ItemsSourceRef.Find( InRangeSelectionEnd );

		RangeStartIndex = FMath::Clamp(RangeStartIndex, 0, ItemsSourceRef.Num()-1);
		RangeEndIndex = FMath::Clamp(RangeEndIndex, 0, ItemsSourceRef.Num()-1);

		if (RangeEndIndex < RangeStartIndex)
		{
			Swap( RangeStartIndex, RangeEndIndex );
		}

		for( int32 ItemIndex = RangeStartIndex; ItemIndex <= RangeEndIndex; ++ItemIndex )
		{
			// check if this item can actually be selected
			if(Private_CanItemBeSelected(ItemsSourceRef[ItemIndex]))
			{
				this->SelectedItems.Add( ItemsSourceRef[ItemIndex] );
			}
		}

		this->InertialScrollManager.ClearScrollVelocity();
	}

	virtual void Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo) override
	{
		STreeView<ItemType>::Private_SignalSelectionChanged(SelectInfo);

		// the SListView does not know about bHideParentsWhenFiltering and will select the boens regardless of their visible
		// ( For example when using select all )
		// this filter out those ones to only keep the ones that can be selected
		{
			TArray<ItemType> FilteredSelection;
			for (const ItemType& Item: this->SelectedItems)
			{
				if (Private_CanItemBeSelected(Item))
				{
					FilteredSelection.Add(Item);
				}
			}
			if (FilteredSelection.Num() != this->SelectedItems.Num())
			{
				this->ClearSelection();
				this->SetItemSelection(FilteredSelection, true, SelectInfo);
			}
		}
	}

	/** Stores sparse item infos */
	void StoreSparseItemInfos()
	{
		CachedSparseItemInfos = STreeView<ItemType>::SparseItemInfos;
	}
	
	/** Restores sparse item infos from last known infos */
	void RestoreSparseItemInfos(const TArray<TSharedPtr<ISkeletonTreeItem>> TreeItems)
	{		
		for (auto It = STreeView<ItemType>::SparseItemInfos.CreateIterator(); It; ++It)
		{
			if (!TreeItems.Contains(It->Key))
			{
				It.RemoveCurrent();
			}
		}
		
		for (const auto& [TreeItem, SparseItemInfo] : CachedSparseItemInfos)
		{
			const TSharedPtr<ISkeletonTreeItem>* TreeItemPtr = TreeItems.FindByPredicate(
				[&TreeItem](const TSharedPtr<ISkeletonTreeItem>& OtherTreeItem)
				{
					return TreeItem->GetRowItemName() == OtherTreeItem->GetRowItemName();
				});
				
			if (TreeItemPtr)
			{
				STreeView<ItemType>::SparseItemInfos.Add(*TreeItemPtr, SparseItemInfo);
			}
		}
	}
	
private:
	/** Holds the last known sparse item infos */
	STreeView<ItemType>::TSparseItemMap CachedSparseItemInfos;
};

void SSkeletonTree::Construct(const FArguments& InArgs, const TSharedRef<FEditableSkeleton>& InEditableSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs)
{
	if (InSkeletonTreeArgs.bHideBonesByDefault)
	{
		BoneFilter = EBoneFilter::None;
	}
	else
	{
		BoneFilter = EBoneFilter::All;
	}
	SocketFilter = ESocketFilter::Active;
	bSelecting = false;

	EditableSkeleton = InEditableSkeleton;
	PreviewScene = InSkeletonTreeArgs.PreviewScene;
	IsEditable = InArgs._IsEditable;
	Mode = InSkeletonTreeArgs.Mode;
	bAllowMeshOperations = InSkeletonTreeArgs.bAllowMeshOperations;
	bAllowSkeletonOperations = InSkeletonTreeArgs.bAllowSkeletonOperations;
	bShowDebugVisualizationOptions = InSkeletonTreeArgs.bShowDebugVisualizationOptions;
	bAllowInvisibleItemSelection = InSkeletonTreeArgs.bAllowInvisibleItemSelection;
	Extenders = InSkeletonTreeArgs.Extenders;
	OnGetFilterText = InSkeletonTreeArgs.OnGetFilterText;
	Builder = InSkeletonTreeArgs.Builder;
	if (!Builder.IsValid())
	{
		Builder = MakeShareable(new FSkeletonTreeBuilder(FSkeletonTreeBuilderArgs()));
	}

	Builder->Initialize(SharedThis(this), InSkeletonTreeArgs.PreviewScene, FOnFilterSkeletonTreeItem::CreateSP(this, &SSkeletonTree::HandleFilterSkeletonTreeItem));

	ContextName = InSkeletonTreeArgs.ContextName;

	TextFilterPtr = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));

	SetPreviewComponentSocketFilter();

	// Register delegates

	if(PreviewScene.IsValid())
	{
		if (TSharedPtr<IPersonaPreviewScene> PreviewScenePtr = PreviewScene.Pin())
		{
			PreviewScenePtr->RegisterOnLODChanged(FSimpleDelegate::CreateSP(this, &SSkeletonTree::OnLODSwitched));
			PreviewScenePtr->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &SSkeletonTree::OnPreviewMeshChanged));
			PreviewScenePtr->RegisterOnSelectedBonesChanged(FOnSelectedBonesChanged::CreateSP(this, &SSkeletonTree::HandleSelectedBonesChanged));
			PreviewScenePtr->RegisterOnSelectedSocketChanged(FOnSelectedSocketChanged::CreateSP(this, &SSkeletonTree::HandleSelectedSocketChanged));
			PreviewScenePtr->RegisterOnDeselectAll(FSimpleDelegate::CreateSP(this, &SSkeletonTree::HandleDeselectAll));

			RegisterOnSelectionChanged(FOnSkeletonTreeSelectionChanged::CreateRaw(PreviewScenePtr.Get(), &IPersonaPreviewScene::HandleSkeletonTreeSelectionChanged));
		}
	}

	if (InSkeletonTreeArgs.OnSelectionChanged.IsBound())
	{
		RegisterOnSelectionChanged(InSkeletonTreeArgs.OnSelectionChanged);
	}

	FCoreUObjectDelegates::OnPackageReloaded.AddSP(this, &SSkeletonTree::HandlePackageReloaded);

	// Create our pinned commands before we bind commands
	IPinnedCommandListModule& PinnedCommandListModule = FModuleManager::LoadModuleChecked<IPinnedCommandListModule>(TEXT("PinnedCommandList"));
	PinnedCommands = PinnedCommandListModule.CreatePinnedCommandList(ContextName);

	// Register and bind all our menu commands
	FSkeletonTreeCommands::Register();
	BindCommands();

	RegisterBlendProfileMenu();
	RegisterNewMenu();
	RegisterFilterMenu();

	this->ChildSlot
	[
		SNew( SOverlay )
		+SOverlay::Slot()
		[
			// Add a border if we are being used as a picker
			SNew(SBorder)
			.Visibility_Lambda([this](){ return Mode == ESkeletonTreeMode::Picker ? EVisibility::Visible: EVisibility::Collapsed; })
			.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		]
		+SOverlay::Slot()
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 2.f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(6.f, 0.0))
				[
					SNew(SPositiveActionButton)
					.Visibility_Lambda([this](){ return bAllowSkeletonOperations ? EVisibility::Visible : EVisibility::Collapsed; })
					.OnGetMenuContent( this, &SSkeletonTree::CreateNewMenuWidget )
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew( NameFilterBox, SSearchBox )
					.SelectAllTextWhenFocused( true )
					.OnTextChanged( this, &SSkeletonTree::OnFilterTextChanged )
					.HintText( LOCTEXT( "SearchBoxHint", "Search Skeleton Tree...") )
					.AddMetaData<FTagMetaData>(TEXT("SkelTree.Search"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(6.f, 0.0))
				.VAlign(VAlign_Center)
				[
					SAssignNew(FilterComboButton, SComboButton)
					.Visibility(InSkeletonTreeArgs.bShowFilterMenu ? EVisibility::Visible : EVisibility::Collapsed)
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
					.ForegroundColor(FSlateColor::UseStyle())
					.ContentPadding(2.0f)
					.OnGetMenuContent( this, &SSkeletonTree::CreateFilterMenuWidget )
					.ToolTipText( this, &SSkeletonTree::GetFilterMenuTooltip )
					.AddMetaData<FTagMetaData>(TEXT("SkelTree.Bones"))
					.HasDownArrow(true)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]

			+ SVerticalBox::Slot()
			.Padding( FMargin( 0.0f, 2.0f, 0.0f, 0.0f ) )
			.AutoHeight()
			[
				PinnedCommands.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.Padding( FMargin( 0.0f, 2.0f, 0.0f, 0.0f ) )
			[
				SAssignNew(TreeHolder, SOverlay)
			]
		]
	];


	SAssignNew(BlendProfilePicker, SBlendProfilePicker, GetEditableSkeleton())
		.Standalone(true)
		.OnBlendProfileSelected(this, &SSkeletonTree::OnBlendProfileSelected);


	CreateTreeColumns();

	SetInitialExpansionState();

	OnLODSwitched();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
SSkeletonTree::~SSkeletonTree()
{
	if (EditableSkeleton.IsValid())
	{
		EditableSkeleton.Pin()->UnregisterOnSkeletonHierarchyChanged(this);
	}
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SSkeletonTree::BindCommands()
{
	// This should not be called twice on the same instance
	check( !UICommandList.IsValid() );

	UICommandList = MakeShareable( new FUICommandList_Pinnable );

	FUICommandList_Pinnable& CommandList = *UICommandList;

	// Grab the list of menu commands to bind...
	const FSkeletonTreeCommands& MenuActions = FSkeletonTreeCommands::Get();

	// ...and bind them all

	CommandList.MapAction(
		MenuActions.FilteringFlattensHierarchy,
		FExecuteAction::CreateLambda([this]() { GetMutableDefault<UPersonaOptions>()->bFlattenSkeletonHierarchyWhenFiltering = !GetDefault<UPersonaOptions>()->bFlattenSkeletonHierarchyWhenFiltering; ApplyFilter(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UPersonaOptions>()->bFlattenSkeletonHierarchyWhenFiltering; }));

	CommandList.MapAction(
		MenuActions.HideParentsWhenFiltering,
		FExecuteAction::CreateLambda([this]() { GetMutableDefault<UPersonaOptions>()->bHideParentsWhenFiltering = !GetDefault<UPersonaOptions>()->bHideParentsWhenFiltering; }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UPersonaOptions>()->bHideParentsWhenFiltering; }));

	CommandList.MapAction(
		MenuActions.ShowBoneIndex,
		FExecuteAction::CreateLambda([this]() { GetMutableDefault<UPersonaOptions>()->bShowBoneIndexes = !GetDefault<UPersonaOptions>()->bShowBoneIndexes; }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UPersonaOptions>()->bShowBoneIndexes; }));
	
	// Bone Filter commands
	CommandList.BeginGroup(TEXT("BoneFilterGroup"));

	CommandList.MapAction(
		MenuActions.ShowAllBones,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetBoneFilter, EBoneFilter::All ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsBoneFilter, EBoneFilter::All ),
		FIsActionButtonVisible::CreateSP( Builder.Get(), &ISkeletonTreeBuilder::IsShowingBones ));

	CommandList.MapAction(
		MenuActions.ShowMeshBones,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetBoneFilter, EBoneFilter::Mesh ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsBoneFilter, EBoneFilter::Mesh ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingBones));

	CommandList.MapAction(
		MenuActions.ShowLODBones,
		FExecuteAction::CreateSP(this, &SSkeletonTree::SetBoneFilter, EBoneFilter::LOD),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SSkeletonTree::IsBoneFilter, EBoneFilter::LOD),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingBones));
	
	CommandList.MapAction(
		MenuActions.ShowWeightedBones,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetBoneFilter, EBoneFilter::Weighted ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsBoneFilter, EBoneFilter::Weighted ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingBones));

	CommandList.MapAction(
		MenuActions.HideBones,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetBoneFilter, EBoneFilter::None ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsBoneFilter, EBoneFilter::None ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingBones));

	CommandList.EndGroup();

	// Socket filter commands
	CommandList.BeginGroup(TEXT("SocketFilterGroup"));

	CommandList.MapAction(
		MenuActions.ShowActiveSockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetSocketFilter, ESocketFilter::Active ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsSocketFilter, ESocketFilter::Active ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingSockets ));

	CommandList.MapAction(
		MenuActions.ShowMeshSockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetSocketFilter, ESocketFilter::Mesh ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsSocketFilter, ESocketFilter::Mesh ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingSockets ));

	CommandList.MapAction(
		MenuActions.ShowSkeletonSockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetSocketFilter, ESocketFilter::Skeleton ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsSocketFilter, ESocketFilter::Skeleton ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingSockets ));

	CommandList.MapAction(
		MenuActions.ShowAllSockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetSocketFilter, ESocketFilter::All ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsSocketFilter, ESocketFilter::All ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingSockets ));

	CommandList.MapAction(
		MenuActions.HideSockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::SetSocketFilter, ESocketFilter::None ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SSkeletonTree::IsSocketFilter, ESocketFilter::None ),
		FIsActionButtonVisible::CreateSP(Builder.Get(), &ISkeletonTreeBuilder::IsShowingSockets ));

	CommandList.EndGroup();

	CommandList.MapAction(
		MenuActions.ShowRetargeting,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnChangeShowingAdvancedOptions),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SSkeletonTree::IsShowingAdvancedOptions),
		FIsActionButtonVisible::CreateLambda([this]() { return Builder->IsShowingBones() && bAllowSkeletonOperations; }));

	CommandList.MapAction(
		MenuActions.ShowDebugVisualization,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnChangeShowingDebugVisualizationOptions),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SSkeletonTree::IsShowingDebugVisualizationOptions),
		FIsActionButtonVisible::CreateLambda([this](){ return bShowDebugVisualizationOptions; }));

	// Socket manipulation commands
	CommandList.MapAction(
		MenuActions.AddSocket,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnAddSocket ),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::IsAddingSocketsAllowed ),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	CommandList.MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnRenameSelected ),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanRenameSelected ) );

	CommandList.MapAction(
		MenuActions.CreateMeshSocket,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnCustomizeSocket ),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowMeshOperations));

	CommandList.MapAction(
		MenuActions.RemoveMeshSocket,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnDeleteSelectedRows ),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanDeleteSelectedRows ),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowMeshOperations));

	CommandList.MapAction(
		MenuActions.PromoteSocketToSkeleton,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnPromoteSocket ), // Adding customization just deletes the mesh socket
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	CommandList.MapAction(
		MenuActions.DeleteSelectedRows,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnDeleteSelectedRows ),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanDeleteSelectedRows ));

	CommandList.MapAction(
		MenuActions.CopyBoneNames,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnCopyBoneNames ));

	static constexpr bool bSelectedOnly = true;
	CommandList.MapAction(
		MenuActions.ResetBoneTransforms,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnResetBoneTransforms, bSelectedOnly ) );

	CommandList.MapAction(
		MenuActions.ResetAllBonesTransforms,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnResetBoneTransforms, !bSelectedOnly ) );

	CommandList.MapAction(
		MenuActions.CopySockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnCopySockets ),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanCopySockets ));

	CommandList.MapAction(
		MenuActions.PasteSockets,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnPasteSockets, false ),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanPasteSockets ));

	CommandList.MapAction(
		MenuActions.PasteSocketsToSelectedBone,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnPasteSockets, true),
		FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanPasteSockets ));

	CommandList.MapAction(
		MenuActions.FocusCamera,
		FExecuteAction::CreateSP(this, &SSkeletonTree::HandleFocusCamera));

	CommandList.MapAction(
		MenuActions.CreateTimeBlendProfile,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnCreateBlendProfile, EBlendProfileMode::TimeFactor),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	CommandList.MapAction(
		MenuActions.CreateWeightBlendProfile,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnCreateBlendProfile, EBlendProfileMode::WeightFactor),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	CommandList.MapAction(
		MenuActions.CreateBlendMask,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnCreateBlendProfile, EBlendProfileMode::BlendMask),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	CommandList.MapAction(
		MenuActions.DeleteCurrentBlendProfile,
		FExecuteAction::CreateSP( this, &SSkeletonTree::OnDeleteCurrentBlendProfile),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	CommandList.MapAction(
		MenuActions.RenameBlendProfile,
		FExecuteAction::CreateSP(this, &SSkeletonTree::OnRenameBlendProfile),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSkeletonTree::ShouldAllowSkeletonOperations));

	PinnedCommands->BindCommandList(UICommandList.ToSharedRef());
}

TSharedRef<ITableRow> SSkeletonTree::MakeTreeRowWidget(TSharedPtr<ISkeletonTreeItem> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );
	
	return InInfo->MakeTreeRowWidget(OwnerTable, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]() { return FilterText; })));
}

void SSkeletonTree::GetFilteredChildren(TSharedPtr<ISkeletonTreeItem> InInfo, TArray< TSharedPtr<ISkeletonTreeItem> >& OutChildren)
{
	check(InInfo.IsValid());
	OutChildren = InInfo->GetFilteredChildren();
}

/** Helper struct for when we rebuild the tree because of a change to its structure */
struct FScopedSavedSelection
{
	FScopedSavedSelection(TSharedPtr<SSkeletonTree> InSkeletonTree)
	: SkeletonTree(InSkeletonTree)
	{
		// record selected items
		if (SkeletonTree.IsValid() && InSkeletonTree->SkeletonTreeView.IsValid())
		{
			TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = InSkeletonTree->SkeletonTreeView->GetSelectedItems();
			for (const TSharedPtr<ISkeletonTreeItem>& SelectedItem : SelectedItems)
			{
				SavedSelections.Add({ SelectedItem->GetRowItemName(), SelectedItem->GetTypeName(), SelectedItem->GetObject() });
			}
		}
	}

	~FScopedSavedSelection()
	{
		if (SkeletonTree.IsValid() && SkeletonTree->SkeletonTreeView.IsValid())
		{
			// restore selection
			for (const TSharedPtr<ISkeletonTreeItem>& Item : SkeletonTree->LinearItems)
			{
				if (Item->GetFilterResult() != ESkeletonTreeFilterResult::Hidden)
				{
					for (FSavedSelection& SavedSelection : SavedSelections)
					{
						if (Item->GetRowItemName() == SavedSelection.ItemName && Item->GetTypeName() == SavedSelection.ItemType && Item->GetObject() == SavedSelection.ItemObject)
						{
							SkeletonTree->SkeletonTreeView->SetItemSelection(Item, true);
							break;
						}
					}
				}
			}
		}
	}

	struct FSavedSelection
	{
		/** Name of the selected item */
		FName ItemName;

		/** Type of the selected item */
		FName ItemType;

		/** Object of selected item */
		UObject* ItemObject;
	};

	TSharedPtr<SSkeletonTree> SkeletonTree;

	TArray<FSavedSelection> SavedSelections;
};

void SSkeletonTree::CreateTreeColumns()
{
	TArray<FName> HiddenColumnsList;
	HiddenColumnsList.Add(ISkeletonTree::Columns::Retargeting);
	HiddenColumnsList.Add(ISkeletonTree::Columns::BlendProfile);
	HiddenColumnsList.Add(ISkeletonTree::Columns::DebugVisualization);

	TSharedRef<SHeaderRow> TreeHeaderRow = 
	SNew(SHeaderRow)
	.CanSelectGeneratedColumn(true)
	.HiddenColumnsList(HiddenColumnsList)

	+ SHeaderRow::Column(ISkeletonTree::Columns::Name)
	.ShouldGenerateWidget(true)
	.DefaultLabel(LOCTEXT("SkeletonBoneNameLabel", "Name"))
	.FillWidth(0.5f)

	+ SHeaderRow::Column(ISkeletonTree::Columns::Retargeting)
	.DefaultLabel(LOCTEXT("SkeletonBoneTranslationRetargetingLabel", "Translation Retargeting"))
	.FillWidth(0.25f)

	+ SHeaderRow::Column(ISkeletonTree::Columns::DebugVisualization)
	.DefaultLabel(LOCTEXT("SkeletonBoneDebugVisualizationLabel", "Debug"))
	.FillWidth(0.25f)

	+ SHeaderRow::Column(ISkeletonTree::Columns::BlendProfile)
	.DefaultLabel(LOCTEXT("BlendProfile", "Blend Profile"))
	.FillWidth(0.25f)
	.OnGetMenuContent(this, &SSkeletonTree::GetBlendProfileColumnMenuContent )
	.HeaderContent()
	[
		SNew(SBox)
		.HeightOverride(24.f)
		.HAlign(HAlign_Left)
		[
			SAssignNew(BlendProfileHeader, SInlineEditableTextBlock)
			.Text_Lambda([this] () -> FText
			{
				FName CurrentProfile = BlendProfilePicker->GetSelectedBlendProfileName();
				return (CurrentProfile != NAME_None) ? FText::FromName(CurrentProfile) : LOCTEXT("NoBlendProfile", "NoBlend");
			})
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
			{
				if (bIsCreateNewBlendProfile)
				{
					BlendProfilePicker->OnCreateNewProfileComitted(InText, InCommitType, NewBlendProfileMode);
					bIsCreateNewBlendProfile = false;
				}
				else if(BlendProfilePicker->GetSelectedBlendProfileName() != NAME_None)
				{
					if (UBlendProfile* Profile = EditableSkeleton.Pin()->RenameBlendProfile(BlendProfilePicker->GetSelectedBlendProfileName(), FName(InText.ToString())))
					{
						BlendProfilePicker->SetSelectedProfile(Profile);
					}
				}
			})
			.OnVerifyTextChanged_Lambda([](const FText& InNewText, FText& OutErrorMessage) -> bool
			{
				return FName::IsValidXName(InNewText.ToString(), INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage);
			})
			.IsReadOnly(true)
		]
	];

	{
		FScopedSavedSelection ScopedSelection(SharedThis(this));

		SkeletonTreeView = SNew(SSkeletonTreeView<TSharedPtr<ISkeletonTreeItem>>)
		.TreeItemsSource(&FilteredItems)
		.OnGenerateRow(this, &SSkeletonTree::MakeTreeRowWidget)
		.OnGetChildren(this, &SSkeletonTree::GetFilteredChildren)
		.OnContextMenuOpening(this, &SSkeletonTree::CreateContextMenu)
		.OnSelectionChanged(this, &SSkeletonTree::OnSelectionChanged)
		.OnIsSelectableOrNavigable(this, &SSkeletonTree::OnIsSelectableOrNavigable)
		.OnItemScrolledIntoView(this, &SSkeletonTree::OnItemScrolledIntoView)
		.OnMouseButtonDoubleClick(this, &SSkeletonTree::OnTreeDoubleClick)
		.OnSetExpansionRecursive(this, &SSkeletonTree::SetTreeItemExpansionRecursive)
		.HighlightParentNodesForSelection(true)
		.AllowInvisibleItemSelection(bAllowInvisibleItemSelection)
		.HeaderRow
		(
			TreeHeaderRow
		);

		TreeHolder->ClearChildren();
		TreeHolder->AddSlot()
		[
			SNew(SScrollBorder, SkeletonTreeView.ToSharedRef())
			[
				SkeletonTreeView.ToSharedRef()
			]
		];
	}

	CreateFromSkeleton();
}

void SSkeletonTree::CreateFromSkeleton()
{	
	SkeletonTreeView->StoreSparseItemInfos();

	// save selected items
	FScopedSavedSelection ScopedSelection(SharedThis(this));

	Items.Empty();
	LinearItems.Empty();
	FilteredItems.Empty();

	FSkeletonTreeBuilderOutput Output(Items, LinearItems);
	Builder->Build(Output);
	ApplyFilter();
	
	SkeletonTreeView->RestoreSparseItemInfos(LinearItems);
}

void SSkeletonTree::ApplyFilter()
{
	TextFilterPtr->SetFilterText(FilterText);

	FilteredItems.Empty();

	FSkeletonTreeFilterArgs FilterArgs(!FilterText.IsEmpty() ? TextFilterPtr : nullptr);
	FilterArgs.bFlattenHierarchyOnFilter = GetDefault<UPersonaOptions>()->bFlattenSkeletonHierarchyWhenFiltering;
	Builder->Filter(FilterArgs, Items, FilteredItems);

	if(!FilterText.IsEmpty())
	{
		for (TSharedPtr<ISkeletonTreeItem>& Item : LinearItems)
		{
			if (Item->GetFilterResult() > ESkeletonTreeFilterResult::Hidden)
			{
				SkeletonTreeView->SetItemExpansion(Item, true);
			}
		}
	}
	else
	{
		SetInitialExpansionState();
	}

	HandleTreeRefresh();
}

void SSkeletonTree::SetInitialExpansionState()
{
	for (TSharedPtr<ISkeletonTreeItem>& Item : LinearItems)
	{
		SkeletonTreeView->SetItemExpansion(Item, Item->IsInitiallyExpanded());
	}
}

TSharedPtr< SWidget > SSkeletonTree::CreateContextMenu()
{
	const FSkeletonTreeCommands& Actions = FSkeletonTreeCommands::Get();

	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection BoneTreeSelection(SelectedItems);

	const bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder( CloseAfterSelection, UICommandList, Extenders );
	{
		if(BoneTreeSelection.HasSelectedOfType<FSkeletonTreeAttachedAssetItem>() || BoneTreeSelection.HasSelectedOfType<FSkeletonTreeSocketItem>() || BoneTreeSelection.HasSelectedOfType<FSkeletonTreeVirtualBoneItem>())
		{
			MenuBuilder.BeginSection("SkeletonTreeSelectedItemsActions", LOCTEXT( "SelectedActions", "Selected Item Actions" ) );
			MenuBuilder.AddMenuEntry( Actions.DeleteSelectedRows );
			MenuBuilder.EndSection();
		}


		const bool bNeedsBoneActionsHeading = BoneTreeSelection.HasSelectedOfType<FSkeletonTreeBoneItem>() || BoneTreeSelection.HasSelectedOfType<FSkeletonTreeVirtualBoneItem>();

		if (bNeedsBoneActionsHeading)
		{
			MenuBuilder.BeginSection("SkeletonTreeBonesAction", LOCTEXT("BoneActions", "Selected Bone Actions"));
		}
		
		const bool bHasBoneSelected = BoneTreeSelection.HasSelectedOfType<FSkeletonTreeBoneItem>();
		const bool bHasVirtualBoneSelected = BoneTreeSelection.HasSelectedOfType<FSkeletonTreeVirtualBoneItem>();
		if (bHasBoneSelected || bHasVirtualBoneSelected)
		{
			MenuBuilder.AddMenuEntry(Actions.CopyBoneNames);

			if (bHasBoneSelected)
			{
				MenuBuilder.AddMenuEntry(Actions.ResetBoneTransforms);
			}

			if (BoneTreeSelection.IsSingleOfTypesSelected<FSkeletonTreeBoneItem, FSkeletonTreeVirtualBoneItem>() && bAllowSkeletonOperations)
			{
				MenuBuilder.AddMenuEntry(Actions.AddSocket);
				MenuBuilder.AddMenuEntry(Actions.PasteSockets);
				MenuBuilder.AddMenuEntry(Actions.PasteSocketsToSelectedBone);
			}
		}

		if (bNeedsBoneActionsHeading)
		{
			if (bAllowSkeletonOperations)
			{
				MenuBuilder.AddSubMenu(LOCTEXT("AddVirtualBone", "Add Virtual Bone"),
					LOCTEXT("AddVirtualBone_ToolTip", "Adds a virtual bone to the skeleton."),
					FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
						{
							TSharedRef<SBoneTreeMenu> MenuContent = SSkeletonTree::CreateVirtualBoneMenu(this);
							MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);
						}));
			}

			MenuBuilder.EndSection();
		}

		if (bAllowSkeletonOperations)
		{
			if(BoneTreeSelection.HasSelectedOfType<FSkeletonTreeBoneItem>())
			{
				UBlendProfile* const SelectedBlendProfile = BlendProfilePicker->GetSelectedBlendProfile();
				if(SelectedBlendProfile && BoneTreeSelection.IsSingleOfTypeSelected<FSkeletonTreeBoneItem>())
				{
					TSharedPtr<FSkeletonTreeBoneItem> BoneItem = BoneTreeSelection.GetSelectedItems<FSkeletonTreeBoneItem>()[0];

					FName BoneName = BoneItem->GetAttachName();
					const USkeleton& Skeleton = GetEditableSkeletonInternal()->GetSkeleton();
					int32 BoneIndex = Skeleton.GetReferenceSkeleton().FindBoneIndex(BoneName);

					float CurrentBlendScale = SelectedBlendProfile->GetBoneBlendScale(BoneIndex);

					MenuBuilder.BeginSection("SkeletonTreeBlendProfileScales", LOCTEXT("BlendProfileContextOptions", "Blend Profile"));
					{
						FUIAction RecursiveSetScales;
						RecursiveSetScales.ExecuteAction = FExecuteAction::CreateSP(this, &SSkeletonTree::RecursiveSetBlendProfileScales, CurrentBlendScale);

						MenuBuilder.AddMenuEntry
						(
							FText::Format(LOCTEXT("RecursiveSetBlendScales_Label", "Recursively Set Blend Scales To {0}"), FText::AsNumber(CurrentBlendScale)),
							LOCTEXT("RecursiveSetBlendScales_ToolTip", "Sets all child bones to use the same blend profile scale as the selected bone"),
							FSlateIcon(),
							RecursiveSetScales
							);
					}
					MenuBuilder.EndSection();
				}

				if(IsShowingAdvancedOptions() && bAllowSkeletonOperations)
				{
					MenuBuilder.BeginSection("SkeletonTreeBoneTranslationRetargeting", LOCTEXT("BoneTranslationRetargetingHeader", "Bone Translation Retargeting"));
					{
						FUIAction RecursiveRetargetingSkeletonAction = FUIAction(FExecuteAction::CreateSP(this, &SSkeletonTree::SetBoneTranslationRetargetingModeRecursive, EBoneTranslationRetargetingMode::Skeleton));
						FUIAction RecursiveRetargetingAnimationAction = FUIAction(FExecuteAction::CreateSP(this, &SSkeletonTree::SetBoneTranslationRetargetingModeRecursive, EBoneTranslationRetargetingMode::Animation));
						FUIAction RecursiveRetargetingAnimationScaledAction = FUIAction(FExecuteAction::CreateSP(this, &SSkeletonTree::SetBoneTranslationRetargetingModeRecursive, EBoneTranslationRetargetingMode::AnimationScaled));
						FUIAction RecursiveRetargetingAnimationRelativeAction = FUIAction(FExecuteAction::CreateSP(this, &SSkeletonTree::SetBoneTranslationRetargetingModeRecursive, EBoneTranslationRetargetingMode::AnimationRelative));
						FUIAction RecursiveRetargetingOrientAndScaleAction = FUIAction(FExecuteAction::CreateSP(this, &SSkeletonTree::SetBoneTranslationRetargetingModeRecursive, EBoneTranslationRetargetingMode::OrientAndScale));

						MenuBuilder.AddMenuEntry
						(LOCTEXT("SetTranslationRetargetingSkeletonChildrenAction", "Recursively Set Translation Retargeting Skeleton")
							, LOCTEXT("BoneTranslationRetargetingSkeletonToolTip", "Use translation from Skeleton.")
							, FSlateIcon()
							, RecursiveRetargetingSkeletonAction
							);

						MenuBuilder.AddMenuEntry
						(LOCTEXT("SetTranslationRetargetingAnimationChildrenAction", "Recursively Set Translation Retargeting Animation")
							, LOCTEXT("BoneTranslationRetargetingAnimationToolTip", "Use translation from animation.")
							, FSlateIcon()
							, RecursiveRetargetingAnimationAction
							);

						MenuBuilder.AddMenuEntry
						(LOCTEXT("SetTranslationRetargetingAnimationScaledChildrenAction", "Recursively Set Translation Retargeting AnimationScaled")
							, LOCTEXT("BoneTranslationRetargetingAnimationScaledToolTip", "Use translation from animation, scale length by Skeleton's proportions.")
							, FSlateIcon()
							, RecursiveRetargetingAnimationScaledAction
							);

						MenuBuilder.AddMenuEntry
						(LOCTEXT("SetTranslationRetargetingAnimationRelativeChildrenAction", "Recursively Set Translation Retargeting AnimationRelative")
							, LOCTEXT("BoneTranslationRetargetingAnimationRelativeToolTip", "Use relative translation from animation similar to an additive animation.")
							, FSlateIcon()
							, RecursiveRetargetingAnimationRelativeAction
							);

						MenuBuilder.AddMenuEntry
						(LOCTEXT("SetTranslationRetargetingOrientAndScaleChildrenAction", "Recursively Set Translation Retargeting OrientAndScale")
							, LOCTEXT("BoneTranslationRetargetingOrientAndScaleToolTip", "Orient And Scale Translation.")
							, FSlateIcon()
							, RecursiveRetargetingOrientAndScaleAction
							);
					}
					MenuBuilder.EndSection();
				}
			}
		}

		if(bAllowMeshOperations)
		{
			MenuBuilder.BeginSection("SkeletonTreeBoneReductionForLOD", LOCTEXT("BoneReductionHeader", "LOD Bone Reduction"));
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("SkeletonTreeBoneReductionForLOD_RemoveSelectedFromLOD", "Remove Selected..."),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateStatic(&SSkeletonTree::CreateMenuForBoneReduction, this, LastCachedLODForPreviewMeshComponent, true)
					);

				MenuBuilder.AddSubMenu(
					LOCTEXT("SkeletonTreeBoneReductionForLOD_RemoveChildrenFromLOD", "Remove Children..."),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateStatic(&SSkeletonTree::CreateMenuForBoneReduction, this, LastCachedLODForPreviewMeshComponent, false)
					);
				
				MenuBuilder.AddSeparator();
				
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SkeletonTreeBoneReductionForLOD_RemoveAllUnskinnedBones", "Remove All Unused")
					, LOCTEXT("SkeletonTreeBoneReductionForLOD_RemoveAllUnskinnedBonesTooltip", "Remove all bones that have no vertices weighted against them from LOD 0 and below. Parent bones, and parents of parents of needed bones will be kept.")
					, FSlateIcon()
					, FUIAction(FExecuteAction::CreateSP(this, &SSkeletonTree::RemoveFromLOD, 0, EBoneReductionSelectionMode::AllUnusedBones, true))
					);
			}
			MenuBuilder.EndSection();
		}

		if (bAllowSkeletonOperations)
		{
			if (BoneTreeSelection.HasSelectedOfType<FSkeletonTreeVirtualBoneItem>())
			{
				MenuBuilder.BeginSection("SkeletonTreeVirtualBoneActions", LOCTEXT("VirtualBoneActions", "Selected Virtual Bone Actions"));

				if (BoneTreeSelection.IsSingleOfTypeSelected<FSkeletonTreeVirtualBoneItem>())
				{
					MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameVirtualBone_Label", "Rename Virtual Bone"), LOCTEXT("RenameVirtualBone_Tooltip", "Rename this virtual bone"));
				}

				MenuBuilder.EndSection();
			}

			if(BoneTreeSelection.HasSelectedOfType<FSkeletonTreeSocketItem>())
			{
				MenuBuilder.BeginSection("SkeletonTreeSocketsActions", LOCTEXT( "SocketActions", "Selected Socket Actions" ) );

				MenuBuilder.AddMenuEntry( Actions.CopySockets );

				if(BoneTreeSelection.IsSingleOfTypeSelected<FSkeletonTreeSocketItem>())
				{
					MenuBuilder.AddMenuEntry( FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameSocket_Label", "Rename Socket"), LOCTEXT("RenameSocket_Tooltip", "Rename this socket") );

					TSharedPtr<FSkeletonTreeSocketItem> SocketItem = BoneTreeSelection.GetSelectedItems<FSkeletonTreeSocketItem>()[0];

					if (SocketItem->IsSocketCustomized() && SocketItem->GetParentType() == ESocketParentType::Mesh && bAllowMeshOperations)
					{
						MenuBuilder.AddMenuEntry( Actions.RemoveMeshSocket );
					}

					// If the socket is on the skeleton, we have a valid mesh
					// and there isn't one of the same name on the mesh, we can customize it
					if (SocketItem->CanCustomizeSocket() )
					{
						if (SocketItem->GetParentType() == ESocketParentType::Skeleton && bAllowMeshOperations)
						{
							MenuBuilder.AddMenuEntry( Actions.CreateMeshSocket );
						}
						else if (SocketItem->GetParentType() == ESocketParentType::Mesh )
						{
							// If a socket is on the mesh only, then offer to promote it to the skeleton
							MenuBuilder.AddMenuEntry( Actions.PromoteSocketToSkeleton );
						}
					}
				}

				MenuBuilder.EndSection();
			}
		}

		if (BoneTreeSelection.HasSelectedOfType<FSkeletonTreeBoneItem>() || BoneTreeSelection.HasSelectedOfType<FSkeletonTreeSocketItem>())
		{
			MenuBuilder.BeginSection("SkeletonTreeAttachedAssets", LOCTEXT( "AttachedAssetsActionsHeader", "Attached Assets Actions" ) );

			if ( BoneTreeSelection.IsSingleItemSelected() )
			{
				MenuBuilder.AddSubMenu(	LOCTEXT( "AttachNewAsset", "Add Preview Asset" ),
					LOCTEXT ( "AttachNewAsset_ToolTip", "Attaches an asset to this part of the skeleton. Assets can also be dragged onto the skeleton from a content browser to attach" ),
					FNewMenuDelegate::CreateSP( this, &SSkeletonTree::FillAttachAssetSubmenu, BoneTreeSelection.GetSingleSelectedItem() ) );
			}

			FUIAction RemoveAllAttachedAssets = FUIAction(	FExecuteAction::CreateSP( this, &SSkeletonTree::OnRemoveAllAssets ),
				FCanExecuteAction::CreateSP( this, &SSkeletonTree::CanRemoveAllAssets ));

			MenuBuilder.AddMenuEntry( LOCTEXT( "RemoveAllAttachedAssets", "Remove All Attached Assets" ),
				LOCTEXT ( "RemoveAllAttachedAssets_ToolTip", "Removes all the attached assets from the skeleton and mesh." ),
				FSlateIcon(), RemoveAllAttachedAssets );

			MenuBuilder.EndSection();
		}

		// Add an empty section so the menu can be extended when there are no optionally-added entries
		MenuBuilder.BeginSection("SkeletonTreeContextMenu");
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

bool GetSourceNameFromItem(TSharedPtr<ISkeletonTreeItem> SourceBone, FName& OutName)
{
	if (SourceBone->IsOfType<FSkeletonTreeBoneItem>())
	{
		OutName = SourceBone->GetRowItemName();
		return true;
	}
	if (SourceBone->IsOfType<FSkeletonTreeVirtualBoneItem>())
	{
		OutName = SourceBone->GetRowItemName();
		return true;
	}
	return false;
}

TSharedRef<SBoneTreeMenu> SSkeletonTree::CreateVirtualBoneMenu(SSkeletonTree* InSkeletonTree)
{
	const TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = InSkeletonTree->GetSelectedItems();

	TSharedRef<SBoneTreeMenu> MenuContent = SNew(SBoneTreeMenu)
		.bShowVirtualBones(false)
		.Title(LOCTEXT("TargetBonePickerTitle", "Pick Target Bone..."))
		.OnBoneSelectionChanged(InSkeletonTree, &SSkeletonTree::OnVirtualTargetBonePicked, SelectedItems)
		.OnGetReferenceSkeleton(InSkeletonTree, &SSkeletonTree::OnGetReferenceSkeleton);

	MenuContent->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda(
		[FilterTextBox = MenuContent->GetFilterTextWidget()](double, float)
		{
			FSlateApplication::Get().SetKeyboardFocus(FilterTextBox);
			return EActiveTimerReturnType::Stop;
		}));

	return MenuContent;
}

void SSkeletonTree::OnVirtualTargetBonePicked(FName TargetBoneName, TArray<TSharedPtr<ISkeletonTreeItem>> SourceBones)
{
	FSlateApplication::Get().DismissAllMenus();

	TArray<FName> VirtualBoneNames;

	for (const TSharedPtr<ISkeletonTreeItem>& SourceBone : SourceBones)
	{
		FName SourceBoneName;
		if(GetSourceNameFromItem(SourceBone, SourceBoneName))
		{
			FName NewVirtualBoneName;
			if(!GetEditableSkeletonInternal()->HandleAddVirtualBone(SourceBoneName, TargetBoneName, NewVirtualBoneName))
			{
				UE_LOGF(LogAnimation, Log, "Could not create space switch bone from %ls to %ls, it already exists", *SourceBoneName.ToString(), *TargetBoneName.ToString());
			}
			else
			{
				VirtualBoneNames.Add(NewVirtualBoneName);
			}
		}
	}

	if (VirtualBoneNames.Num() > 0)
	{
		CreateFromSkeleton();
		SkeletonTreeView->ClearSelection();

		TSharedPtr<ISkeletonTreeItem> LastItem;
		for (TSharedPtr<ISkeletonTreeItem> SkeletonRow : LinearItems)
		{
			if (SkeletonRow->IsOfType<FSkeletonTreeVirtualBoneItem>())
			{
				LastItem = SkeletonRow;
				FName RowName = SkeletonRow->GetRowItemName();
				for (const FName& VB : VirtualBoneNames)
				{
					if (RowName == VB)
					{
						SkeletonTreeView->SetItemSelection(SkeletonRow, true);
						SkeletonTreeView->RequestScrollIntoView(SkeletonRow);
						break;
					}
				}
			}
		}

		if (LastItem.IsValid())
		{
			SkeletonTreeView->RequestScrollIntoView(LastItem);
		}
	}
}

void SSkeletonTree::CreateMenuForBoneReduction(FMenuBuilder& MenuBuilder, SSkeletonTree * Widget, int32 LODIndex, bool bIncludeSelected)
{
	const EBoneReductionSelectionMode SelectionMode = bIncludeSelected ? EBoneReductionSelectionMode::SelectedBones : EBoneReductionSelectionMode::ChildrenOfSelectedBones;
	MenuBuilder.AddMenuEntry
	(FText::FromString(FString::Printf(TEXT("From LOD %d and below"), LODIndex))
		, FText::FromString(FString::Printf(TEXT("Remove Selected %s from current LOD %d and all lower LODs"), (bIncludeSelected) ? TEXT("bones") : TEXT("children"), LODIndex))
		, FSlateIcon()
		, FUIAction(FExecuteAction::CreateSP(Widget, &SSkeletonTree::RemoveFromLOD, LODIndex, SelectionMode, true))
		);

	MenuBuilder.AddMenuEntry
	(FText::FromString(FString::Printf(TEXT("From LOD %d only"), LODIndex))
		, FText::FromString(FString::Printf(TEXT("Remove selected %s from current LOD %d only"), (bIncludeSelected) ? TEXT("bones") : TEXT("children"), LODIndex))
		, FSlateIcon()
		, FUIAction(FExecuteAction::CreateSP(Widget, &SSkeletonTree::RemoveFromLOD, LODIndex, SelectionMode, false))
		);
}


void SSkeletonTree::SetBoneTranslationRetargetingModeRecursive(EBoneTranslationRetargetingMode::Type NewRetargetingMode)
{
	TArray<FName> BoneNames;
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);
	for (const TSharedPtr<FSkeletonTreeBoneItem>& Item : TreeSelection.GetSelectedItems<FSkeletonTreeBoneItem>())
	{
		BoneNames.Add(Item->GetRowItemName());
	}

	GetEditableSkeletonInternal()->SetBoneTranslationRetargetingModeRecursive(BoneNames, NewRetargetingMode);
}

void SSkeletonTree::RemoveFromLOD(int32 LODIndex, EBoneReductionSelectionMode SelectionMode, bool bIncludeBelowLODs)
{
	// we cant do this without a preview scene
	if (!GetPreviewScene().IsValid())
	{
		return;
	}

	UDebugSkelMeshComponent* PreviewMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
	USkeletalMesh* SkeletalMesh = PreviewMeshComponent->GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return;
	}

	// ask users you can't undo this change, and warn them
	const FText Message(LOCTEXT("RemoveBonesFromLODWarning", "This action can't be undone. Would you like to continue?"));
	if (FMessageDialog::Open(EAppMsgType::YesNo, Message) == EAppReturnType::Yes)
	{
		TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
		FSkeletonTreeSelection TreeSelection(SelectedItems);
		const FReferenceSkeleton& RefSkeleton = GetEditableSkeletonInternal()->GetSkeleton().GetReferenceSkeleton();
		
		IMeshBoneReductionModule& MeshBoneReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshBoneReductionModule>("MeshBoneReduction");
		IMeshBoneReduction& MeshBoneReduction = *MeshBoneReductionModule.GetMeshBoneReductionInterface();
		
		TArray<FName> BonesToRemove;
		TArray<FName> ForceKeepBones;

		if (SelectionMode == EBoneReductionSelectionMode::AllUnusedBones)
		{
			MeshBoneReduction.BuildBonesToBeRemovedUsedBySkinWeights(SkeletalMesh, ForceKeepBones, BonesToRemove);
		}
		else
		{
			for (const TSharedPtr<FSkeletonTreeBoneItem>& Item : TreeSelection.GetSelectedItems<FSkeletonTreeBoneItem>())
			{
				FName BoneName = Item->GetRowItemName();
				int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
				if (BoneIndex != INDEX_NONE)
				{
					if (SelectionMode == EBoneReductionSelectionMode::SelectedBones)
					{
						BonesToRemove.AddUnique(BoneName);
					}
					else
					{
						// Add all child bones.
						for (int32 ChildIndex = BoneIndex + 1; ChildIndex < RefSkeleton.GetRawBoneNum(); ++ChildIndex)
						{
							if (RefSkeleton.GetParentIndex(ChildIndex) == BoneIndex)
							{
								FName ChildBoneName = RefSkeleton.GetBoneName(ChildIndex);
								BonesToRemove.AddUnique(ChildBoneName);
							}
						}
					}
				}
			}
		}

		MeshBoneReduction.ReduceBoneCounts(SkeletalMesh, ForceKeepBones, BonesToRemove, LODIndex, bIncludeBelowLODs);

		// update UI to reflect the change
		OnLODSwitched();
	}
}

void SSkeletonTree::OnCopyBoneNames()
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);

	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedBones = TreeSelection.GetSelectedItemsOfTypes<FSkeletonTreeBoneItem, FSkeletonTreeVirtualBoneItem>();

	if( SelectedBones.Num() > 0 )
	{
		bool bFirst = true;
		FString BoneNames;
		for (const TSharedPtr<ISkeletonTreeItem>& Item : SelectedBones)
		{
			FName BoneName = Item->GetRowItemName();
			if (!bFirst)
			{
				BoneNames += "\r\n";
			}
			BoneNames += BoneName.ToString();
			bFirst = false;
		}
		FPlatformApplicationMisc::ClipboardCopy( *BoneNames );
	}
}

void SSkeletonTree::OnResetBoneTransforms(const bool bSelectedOnly)
{
	if (GetPreviewScene().IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
		check(PreviewComponent);
		UAnimPreviewInstance* PreviewInstance = PreviewComponent->PreviewInstance;
		check(PreviewInstance);

		const TArray<FAnimNode_ModifyBone>& ModifyBones = PreviewInstance->GetBoneControllers();
		if (ModifyBones.IsEmpty())
		{
			return;
		}
		
		if (!bSelectedOnly)
		{
			GEditor->BeginTransaction(LOCTEXT("SkeletonTree_ResetAllBonesTransforms", "Reset All Bone Transforms"));
			{
				PreviewInstance->SetFlags(RF_Transactional);
				PreviewInstance->Modify();
				PreviewInstance->ResetModifiedBone();
			}
			GEditor->EndTransaction();
			return;
		}
		
		TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
		FSkeletonTreeSelection TreeSelection(SelectedItems);
		TArray<TSharedPtr<FSkeletonTreeBoneItem>> SelectedBones = TreeSelection.GetSelectedItems<FSkeletonTreeBoneItem>();
		if (SelectedBones.Num() > 0)
		{
			bool bModified = false;
			GEditor->BeginTransaction(LOCTEXT("SkeletonTree_ResetBoneTransforms", "Reset Bone Transforms"));

			for (const TSharedPtr<FSkeletonTreeBoneItem>& Item : SelectedBones)
			{
				FName BoneName = Item->GetRowItemName();
				const FAnimNode_ModifyBone* ModifiedBone = PreviewInstance->FindModifiedBone(BoneName);
				if (ModifiedBone != nullptr)
				{
					if (!bModified)
					{
						PreviewInstance->SetFlags(RF_Transactional);
						PreviewInstance->Modify();
						bModified = true;
					}

					PreviewInstance->RemoveBoneModification(BoneName);
				}
			}

			GEditor->EndTransaction();
		}
	}
}

void SSkeletonTree::OnCopySockets() const
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);
	TArray<TSharedPtr<FSkeletonTreeSocketItem>> SelectedSockets = TreeSelection.GetSelectedItems<FSkeletonTreeSocketItem>();
	int32 NumSocketsToCopy = SelectedSockets.Num();
	if ( NumSocketsToCopy > 0 )
	{
		FString SocketsDataString;

		for (const TSharedPtr<FSkeletonTreeSocketItem>& Item : SelectedSockets)
		{
			SocketsDataString += SerializeSocketToString(Item->GetSocket(), Item->GetParentType());
		}

		FString CopyString = FString::Printf( TEXT("%s\nNumSockets=%d\n%s"), *FEditableSkeleton::SocketCopyPasteHeader, NumSocketsToCopy, *SocketsDataString );

		FPlatformApplicationMisc::ClipboardCopy( *CopyString );
	}
}

bool SSkeletonTree::CanCopySockets() const
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);
	TArray<TSharedPtr<FSkeletonTreeSocketItem>> SelectedSockets = TreeSelection.GetSelectedItems<FSkeletonTreeSocketItem>();
	return SelectedSockets.Num() > 0;
}

FString SSkeletonTree::SerializeSocketToString( USkeletalMeshSocket* Socket, ESocketParentType ParentType) const
{
	FString SocketString;

	SocketString += FString::Printf( TEXT( "IsOnSkeleton=%s\n" ), ParentType == ESocketParentType::Skeleton ? TEXT( "1" ) : TEXT( "0" ) );

	FStringOutputDevice Buffer;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice( &Context, Socket, nullptr, Buffer, TEXT( "copy" ), 0, PPF_Copy, false );
	SocketString += Buffer;

	return SocketString;
}

void SSkeletonTree::OnPasteSockets(bool bPasteToSelectedBone)
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);

	// Pasting sockets should only work if there is just one bone selected
	if ( TreeSelection.IsSingleOfTypesSelected<FSkeletonTreeBoneItem, FSkeletonTreeVirtualBoneItem>())
	{
		FName DestBoneName = bPasteToSelectedBone ? TreeSelection.GetSingleSelectedItem()->GetRowItemName() : NAME_None;
		USkeletalMesh* SkeletalMesh = GetPreviewScene().IsValid() ? ToRawPtr(GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;
		GetEditableSkeletonInternal()->HandlePasteSockets(DestBoneName, SkeletalMesh);

		CreateFromSkeleton();
	}
}

bool SSkeletonTree::CanPasteSockets() const
{
	if(SkeletonTreeView->GetNumItemsSelected() == 1)
	{
		TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
		FSkeletonTreeSelection TreeSelection(SelectedItems);

		return TreeSelection.IsSingleOfTypesSelected<FSkeletonTreeBoneItem, FSkeletonTreeVirtualBoneItem>();
	}

	return false;
}

void SSkeletonTree::OnAddSocket()
{
	// This adds a socket to the currently selected bone in the SKELETON, not the MESH.
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);

	// Can only add a socket to one bone
	if (TreeSelection.IsSingleOfTypesSelected<FSkeletonTreeBoneItem, FSkeletonTreeVirtualBoneItem>())
	{
		FName BoneName = TreeSelection.GetSingleSelectedItem()->GetRowItemName();
		USkeletalMeshSocket* NewSocket = GetEditableSkeletonInternal()->HandleAddSocket(BoneName);
		CreateFromSkeleton();

		FSelectedSocketInfo SocketInfo(NewSocket, true);
		SetSelectedSocket(SocketInfo);

		// now let us choose the socket name
		for (TSharedPtr<ISkeletonTreeItem>& Item : LinearItems)
		{
			if (Item->IsOfType<FSkeletonTreeSocketItem>())
			{
				if (Item->GetRowItemName() == NewSocket->SocketName)
				{
					OnRenameSelected();
					break;
				}
			}
		}
	}
}

void SSkeletonTree::OnCustomizeSocket()
{
	// This should only be called on a skeleton socket, it copies the 
	// socket to the mesh so the user can edit it separately
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);

	if(TreeSelection.IsSingleOfTypeSelected<FSkeletonTreeSocketItem>())
	{
		USkeletalMeshSocket* SocketToCustomize = StaticCastSharedPtr<FSkeletonTreeSocketItem>(TreeSelection.GetSingleSelectedItem())->GetSocket();
		USkeletalMesh* SkeletalMesh = GetPreviewScene().IsValid() ? ToRawPtr(GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;
		GetEditableSkeletonInternal()->HandleCustomizeSocket(SocketToCustomize, SkeletalMesh);
		CreateFromSkeleton();
	}
}

void SSkeletonTree::OnPromoteSocket()
{
	// This should only be called on a mesh socket, it copies the 
	// socket to the skeleton so all meshes can use it
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);

	// Can only customize one socket (CreateContextMenu() should prevent this firing!)
	if(TreeSelection.IsSingleOfTypeSelected<FSkeletonTreeSocketItem>())
	{
		USkeletalMeshSocket* SocketToPromote = StaticCastSharedPtr<FSkeletonTreeSocketItem>(TreeSelection.GetSingleSelectedItem())->GetSocket();
		GetEditableSkeletonInternal()->HandlePromoteSocket(SocketToPromote);
		CreateFromSkeleton();
	}
}

void SSkeletonTree::FillAttachAssetSubmenu(FMenuBuilder& MenuBuilder, const TSharedPtr<ISkeletonTreeItem> TargetItem)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	TArray<UClass*> FilterClasses = FComponentAssetBrokerage::GetSupportedAssets(USceneComponent::StaticClass());

	//Clean up the selection so it is relevant to Persona
	FilterClasses.RemoveSingleSwap(UBlueprint::StaticClass(), EAllowShrinking::No); //Child actor components broker gives us blueprints which isn't wanted
	FilterClasses.RemoveSingleSwap(USoundBase::StaticClass(), EAllowShrinking::No); //No sounds wanted

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	for(int i = 0; i < FilterClasses.Num(); ++i)
	{
		AssetPickerConfig.Filter.ClassPaths.Add(FilterClasses[i]->GetClassPathName());
	}


	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SSkeletonTree::OnAssetSelectedFromPicker, TargetItem);

	TSharedRef<SWidget> MenuContent = SNew(SBox)
	.WidthOverride(384.f)
	.HeightOverride(500.f)
	[
		ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
	MenuBuilder.AddWidget( MenuContent, FText::GetEmpty(), true);
}

void SSkeletonTree::OnAssetSelectedFromPicker(const FAssetData& AssetData, const TSharedPtr<ISkeletonTreeItem> TargetItem)
{
	FSlateApplication::Get().DismissAllMenus();
	TArray<FAssetData> Assets;
	Assets.Add(AssetData);

	AttachAssets(TargetItem.ToSharedRef(), Assets);
}

void  SSkeletonTree::OnRemoveAllAssets()
{
	GetEditableSkeletonInternal()->HandleRemoveAllAssets(GetPreviewScene());

	CreateFromSkeleton();
}

bool SSkeletonTree::CanRemoveAllAssets() const
{
	USkeletalMesh* SkeletalMesh = GetPreviewScene().IsValid() ? ToRawPtr(GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;

	const bool bHasPreviewAttachedObjects = GetEditableSkeletonInternal()->GetSkeleton().PreviewAttachedAssetContainer.Num() > 0;
	const bool bHasMeshPreviewAttachedObjects = ( SkeletalMesh && SkeletalMesh->GetPreviewAttachedAssetContainer().Num() );

	return bHasPreviewAttachedObjects || bHasMeshPreviewAttachedObjects;
}

bool SSkeletonTree::CanRenameSelected() const
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();

	return SelectedItems.Num() == 1 && SelectedItems[0]->CanRenameItem();
}

void SSkeletonTree::OnRenameSelected()
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();

	if(SelectedItems.Num() == 1 && SelectedItems[0]->CanRenameItem())
	{
		SkeletonTreeView->RequestScrollIntoView(SelectedItems[0]);
		DeferredRenameRequest = SelectedItems[0];
	}
}

bool SSkeletonTree::OnIsSelectableOrNavigable(TSharedPtr<class ISkeletonTreeItem> InItem) const
{
	return InItem && InItem->GetFilterResult() == ESkeletonTreeFilterResult::Shown;
}

void SSkeletonTree::OnSelectionChanged(TSharedPtr<ISkeletonTreeItem> Selection, ESelectInfo::Type SelectInfo)
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();

	if( Selection.IsValid() )
	{
		// Disable bone proxy ticking on all bone/virtual bones
		for (TSharedPtr<ISkeletonTreeItem>& Item : LinearItems)
		{
			if (Item->IsOfType<FSkeletonTreeBoneItem>())
			{
				StaticCastSharedPtr<FSkeletonTreeBoneItem>(Item)->EnableBoneProxyTick(false);
			}
			else if (Item->IsOfType<FSkeletonTreeVirtualBoneItem>())
			{
				StaticCastSharedPtr<FSkeletonTreeVirtualBoneItem>(Item)->EnableBoneProxyTick(false);
			}
		}

		//Get all the selected items
		FSkeletonTreeSelection TreeSelection(SelectedItems);

		if (GetPreviewScene().IsValid())
		{
			UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
			if (TreeSelection.SelectedItems.Num() > 0 && PreviewComponent)
			{
				// pick the first settable bone from the selection
				for (TSharedPtr<ISkeletonTreeItem> Item : TreeSelection.SelectedItems)
				{
					if((Item->IsOfType<FSkeletonTreeBoneItem>() || Item->IsOfType<FSkeletonTreeVirtualBoneItem>()))
					{
						// enable ticking on the selected bone proxies
						if (Item->IsOfType<FSkeletonTreeBoneItem>())
						{
							StaticCastSharedPtr<FSkeletonTreeBoneItem>(Item)->EnableBoneProxyTick(true);
						}
						else if (Item->IsOfType<FSkeletonTreeVirtualBoneItem>())
						{
							StaticCastSharedPtr<FSkeletonTreeVirtualBoneItem>(Item)->EnableBoneProxyTick(true);
						}

						// Test SelectInfo so we don't end up in an infinite loop due to delegates calling each other
						if (SelectInfo != ESelectInfo::Direct)
						{
							FName BoneName = Item->GetRowItemName();

							// Get bone index
							int32 BoneIndex = PreviewComponent->GetBoneIndex(BoneName);
							if (BoneIndex != INDEX_NONE)
							{
								GetPreviewScene()->SetSelectedBone(BoneName, SelectInfo);
								break;
							}
						}
					}
					// Test SelectInfo so we don't end up in an infinite loop due to delegates calling each other
					else if (SelectInfo != ESelectInfo::Direct && Item->IsOfType<FSkeletonTreeSocketItem>())
					{
						TSharedPtr<FSkeletonTreeSocketItem> SocketItem = StaticCastSharedPtr<FSkeletonTreeSocketItem>(Item);
						USkeletalMeshSocket* Socket = SocketItem->GetSocket();
						FSelectedSocketInfo SocketInfo(Socket, SocketItem->GetParentType() == ESocketParentType::Skeleton);
						GetPreviewScene()->SetSelectedSocket(SocketInfo);
					}
					else if (Item->IsOfType<FSkeletonTreeAttachedAssetItem>())
					{
						GetPreviewScene()->DeselectAll();
					}
				}
				PreviewComponent->PostInitMeshObject(PreviewComponent->MeshObject);
			}
		}
	}
	else
	{
		if (GetPreviewScene().IsValid())
		{
			// Tell the preview scene if the user ctrl-clicked the selected bone/socket to de-select it
			GetPreviewScene()->DeselectAll();
		}
	}

	TArrayView<TSharedPtr<ISkeletonTreeItem>> ArrayView(SelectedItems);
	OnSelectionChangedMulticast.Broadcast(ArrayView, SelectInfo);
}

void SSkeletonTree::AttachAssets(const TSharedRef<ISkeletonTreeItem>& TargetItem, const TArray<FAssetData>& AssetData)
{
	bool bAllAssetWereLoaded = true;
	TArray<UObject*> DroppedObjects;
	for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
	{
		UObject* Object = AssetData[AssetIdx].GetAsset();
		if ( Object != NULL )
		{
			if (FComponentAssetBrokerage::GetPrimaryComponentForAsset(Object->GetClass()) != nullptr)
			{
				DroppedObjects.Add(Object);
			}
		}
		else
		{
			bAllAssetWereLoaded = false;
		}
	}

	if(bAllAssetWereLoaded)
	{
		FName AttachToName = TargetItem->GetAttachName();
		bool bAttachToMesh = TargetItem->IsOfType<FSkeletonTreeSocketItem>() &&
		StaticCastSharedRef<FSkeletonTreeSocketItem>(TargetItem)->GetParentType() == ESocketParentType::Mesh;
		
		GetEditableSkeletonInternal()->HandleAttachAssets(DroppedObjects, AttachToName, bAttachToMesh, GetPreviewScene());
		CreateFromSkeleton();
	}
}

void SSkeletonTree::OnItemScrolledIntoView( TSharedPtr<ISkeletonTreeItem> InItem, const TSharedPtr<ITableRow>& InWidget)
{
	if(DeferredRenameRequest.IsValid())
	{
		DeferredRenameRequest->RequestRename();
		DeferredRenameRequest.Reset();
	}
}

void SSkeletonTree::OnTreeDoubleClick( TSharedPtr<ISkeletonTreeItem> InItem )
{
	InItem->OnItemDoubleClicked();
}

void SSkeletonTree::SetTreeItemExpansionRecursive(TSharedPtr< ISkeletonTreeItem > TreeItem, bool bInExpansionState) const
{
	SkeletonTreeView->SetItemExpansion(TreeItem, bInExpansionState);

	// Recursively go through the children.
	for (auto It = TreeItem->GetChildren().CreateIterator(); It; ++It)
	{
		SetTreeItemExpansionRecursive(*It, bInExpansionState);
	}
}

void SSkeletonTree::PostUndo(bool bSuccess)
{
	// Rebuild the tree view whenever we undo a change to the skeleton
	CreateFromSkeleton();
	HandleTreeRefresh();
}

void SSkeletonTree::PostRedo(bool bSuccess)
{
	// Rebuild the tree view whenever we redo a change to the skeleton
	CreateFromSkeleton();
	HandleTreeRefresh();
}

void SSkeletonTree::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	ApplyFilter();
}

void SSkeletonTree::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (USkeleton* NewObject = Cast<USkeleton>(RepointedObjectPair.Value))
			{
				if (&GetEditableSkeletonInternal()->GetSkeleton() == NewObject)
				{
					Refresh();
				}
			}
		}
	}
}

TSharedRef<SWidget> SSkeletonTree::GetBlendProfileColumnMenuContent()
{
	FToolMenuContext MenuContext(UICommandList, Extenders);
	USkeletonTreeMenuContext* SkeletonTreeMenuContext = NewObject<USkeletonTreeMenuContext>();
	SkeletonTreeMenuContext->SkeletonTree = SharedThis(this);
	MenuContext.AddObject(SkeletonTreeMenuContext);

	return UToolMenus::Get()->GenerateWidget("SkeletonTree.BlendProfilesMenu", MenuContext);
}

void SSkeletonTree::ExpandTreeOnSelection(TSharedPtr<ISkeletonTreeItem> RowToExpand, bool bForce)
{
	if(GetDefault<UPersonaOptions>()->bExpandTreeOnSelection || bForce)
	{
		RowToExpand = RowToExpand->GetParent();
		while(RowToExpand.IsValid())
		{
			SkeletonTreeView->SetItemExpansion(RowToExpand, true);
			RowToExpand = RowToExpand->GetParent();
		}
	}
}

void SSkeletonTree::RegisterBlendProfileMenu()
{
	const FName MenuName("SkeletonTree.BlendProfilesMenu");
	if (UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

	Menu->AddDynamicSection(NAME_None,
		FNewSectionConstructChoice(FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			CreateBlendProfileMenu(InMenu);
		})));
}

void SSkeletonTree::CreateBlendProfileMenu(UToolMenu* InMenu)
{
	USkeletonTreeMenuContext* MenuContext = InMenu->Context.FindContext<USkeletonTreeMenuContext>();
	if(MenuContext == nullptr)
	{
		return;
	}

	TSharedPtr<SSkeletonTree> SkeletonTree = MenuContext->SkeletonTree.Pin();
	if(!SkeletonTree.IsValid())
	{
		return;
	}
	
	const FSkeletonTreeCommands& Actions = FSkeletonTreeCommands::Get();
	
	static const FName BlendProfileSectionNames[]
	{
		TEXT("BlendProfileTimeActions"),
		TEXT("BlendProfileWeightActions"),
		TEXT("BlendMaskActions")
	};

	InMenu->AddSection(BlendProfileSectionNames[0], LOCTEXT("BlendProfilesTime", "Blend Profiles - Time"));
	InMenu->AddSection(BlendProfileSectionNames[1], LOCTEXT("BlendProfilesWeight", "Blend Profiles - Weight"));
	InMenu->AddSection(BlendProfileSectionNames[2], LOCTEXT("BlendProfiles", "Blend Masks"));
	FToolMenuSection* BlendProfileSections[] = 
	{
		InMenu->FindSection(BlendProfileSectionNames[0]),
		InMenu->FindSection(BlendProfileSectionNames[1]),
		InMenu->FindSection(BlendProfileSectionNames[2])
	};

	static const FText SelectBlendProfileToolTipText = LOCTEXT("SelectBlendProfileTooltip", "Select this blend profile for editing.");
	static const FText SelectBlendMaskToolTipText = LOCTEXT("SelectBlendMaskTooltip", "Select this blend mask for editing.");
	static const FText SelectBlendProfileToolTipTexts[] = 
	{
		SelectBlendProfileToolTipText,
		SelectBlendProfileToolTipText,
		SelectBlendMaskToolTipText
	};	

	UEnum* ModeEnum = StaticEnum<EBlendProfileMode>();
	check(ModeEnum);

	for (UBlendProfile* Profile : SkeletonTree->GetEditableSkeletonInternal()->GetBlendProfiles())
	{
		if (Profile)
		{
			int32 EnumIndex = ModeEnum->GetIndexByValue((int64)Profile->GetMode());
			BlendProfileSections[EnumIndex]->AddMenuEntry(
						Profile->GetFName(),
						FText::FromName(Profile->GetFName()),
						SelectBlendProfileToolTipTexts[EnumIndex],
						FSlateIcon(),
						FToolUIActionChoice(
							FUIAction(
								FExecuteAction::CreateSP(SkeletonTree->BlendProfilePicker.ToSharedRef(), &SBlendProfilePicker::SetSelectedProfile, Profile, true),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(SkeletonTree.Get(), &SSkeletonTree::IsBlendProfileSelected, Profile->GetFName())
							)
						),
						EUserInterfaceActionType::RadioButton
					);
		}
	}

	if (SkeletonTree->BlendProfilePicker->GetSelectedBlendProfileName() != NAME_None)
	{
		FToolMenuSection& EditSection = InMenu->AddSection(TEXT("BlendProfileEdit"), LOCTEXT("EditBlendProfilesSection", "Edit"));

		EditSection.AddMenuEntry(
			"ClearBlendProfile",
			LOCTEXT("Clear", "Clear Selected"),
			LOCTEXT("Clear_ToolTip", "Clear the selected blend profile/mask."),
			FSlateIcon(),
			FToolUIActionChoice(FUIAction(FExecuteAction::CreateSP(SkeletonTree->BlendProfilePicker.ToSharedRef(), &SBlendProfilePicker::OnClearSelection))));

		EditSection.AddMenuEntry(
			Actions.RenameBlendProfile,
			FText::Format(LOCTEXT("RenameBlendProfileLabel", "Rename {0}"),
				FText::FromName(SkeletonTree->BlendProfilePicker->GetSelectedBlendProfileName())));

		EditSection.AddMenuEntry(
			Actions.DeleteCurrentBlendProfile,
			FText::Format(LOCTEXT("DeleteBlendProfileLabel", "Delete {0}"),
				FText::FromName(SkeletonTree->BlendProfilePicker->GetSelectedBlendProfileName())));
	}

	{
		FToolMenuSection& NewSection = InMenu->AddSection(TEXT("BlendProfileNew"), LOCTEXT("NewBlendProfiles", "New"));
		NewSection.AddMenuEntry(Actions.CreateTimeBlendProfile);
		NewSection.AddMenuEntry(Actions.CreateWeightBlendProfile);
		NewSection.AddMenuEntry(Actions.CreateBlendMask);
	}
}

void SSkeletonTree::OnCreateBlendProfile(const EBlendProfileMode InMode)
{
	// Ensure the Blend Profile Column is Visible
	BlendProfilePicker->OnClearSelection();
	SkeletonTreeView->GetHeaderRow()->SetShowGeneratedColumn(ISkeletonTree::Columns::BlendProfile);

	// Set our NewBlendProfileMode for our BlendProfileHeader to use when the text is commited.
	NewBlendProfileMode = InMode;

	// Activate the Header Entry Box
	BlendProfileHeader->SetReadOnly(false);
	BlendProfileHeader->EnterEditingMode();
	bIsCreateNewBlendProfile = true;
}

void SSkeletonTree::OnDeleteCurrentBlendProfile()
{
	GetEditableSkeletonInternal()->RemoveBlendProfile(BlendProfilePicker->GetSelectedBlendProfile());
	BlendProfilePicker->OnClearSelection();
}

void SSkeletonTree::OnRenameBlendProfile()
{
	// Activate the Header Entry Box
	BlendProfileHeader->SetReadOnly(false);
	BlendProfileHeader->EnterEditingMode();
}

bool SSkeletonTree::IsBlendProfileSelected(FName ProfileName) const
{
	return BlendProfilePicker->GetSelectedBlendProfileName() == ProfileName;
}

void SSkeletonTree::Refresh()
{
	CreateFromSkeleton();
}

void SSkeletonTree::RefreshFilter()
{
	ApplyFilter();
}

void SSkeletonTree::SetSkeletalMesh(USkeletalMesh* NewSkeletalMesh)
{
	if (GetPreviewScene().IsValid())
	{
		GetPreviewScene()->SetPreviewMesh(NewSkeletalMesh);
	}

	CreateFromSkeleton();
}

void SSkeletonTree::SetSelectedSocket( const FSelectedSocketInfo& SocketInfo )
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		// Firstly, find which row (if any) contains the socket requested
		for (auto SkeletonRowIt = LinearItems.CreateConstIterator(); SkeletonRowIt; ++SkeletonRowIt)
		{
			TSharedPtr<ISkeletonTreeItem> SkeletonRow = *(SkeletonRowIt);

			if (SkeletonRow->GetFilterResult() != ESkeletonTreeFilterResult::Hidden && SkeletonRow->IsOfType<FSkeletonTreeSocketItem>() && StaticCastSharedPtr<FSkeletonTreeSocketItem>(SkeletonRow)->GetSocket() == SocketInfo.Socket)
			{
				SkeletonTreeView->SetItemSelection(SkeletonRow, true);
				ExpandTreeOnSelection(SkeletonRow);
				SkeletonTreeView->RequestScrollIntoView(SkeletonRow);
			}
		}
	}
}

void SSkeletonTree::SetSelectedBone( const FName& BoneName, ESelectInfo::Type InSelectInfo )
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		// Find which row (if any) contains the bone requested
		for (auto SkeletonRowIt = LinearItems.CreateConstIterator(); SkeletonRowIt; ++SkeletonRowIt)
		{
			TSharedPtr<ISkeletonTreeItem> SkeletonRow = *(SkeletonRowIt);

			if (SkeletonRow->GetFilterResult() != ESkeletonTreeFilterResult::Hidden && (SkeletonRow->IsOfType<FSkeletonTreeBoneItem>() || SkeletonRow->IsOfType<FSkeletonTreeVirtualBoneItem>()) && SkeletonRow->GetRowItemName() == BoneName)
			{
				SkeletonTreeView->SetItemSelection(SkeletonRow, true, InSelectInfo);
				ExpandTreeOnSelection(SkeletonRow);
				SkeletonTreeView->RequestScrollIntoView(SkeletonRow);
			}
		}
	}
}

void SSkeletonTree::SetSelectedBones(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo)
{
	if (InBoneNames.IsEmpty())
	{
		return;
	}
	
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		const TArray<TSharedPtr<ISkeletonTreeItem>> SkeletonTreeItems = LinearItems.FilterByPredicate([&InBoneNames](const TSharedPtr<ISkeletonTreeItem>& SkeletonRow)
		{
			if (SkeletonRow->GetFilterResult() == ESkeletonTreeFilterResult::Hidden)
			{
				return false;
			}

			if (SkeletonRow->IsOfType<FSkeletonTreeBoneItem>() || SkeletonRow->IsOfType<FSkeletonTreeVirtualBoneItem>())
			{
				return InBoneNames.Contains(SkeletonRow->GetRowItemName());	
			}

			return false;
		});

		if (SkeletonTreeItems.IsEmpty())
		{
			return;
		}

		ExpandTreeOnSelection(SkeletonTreeItems[0]);
		constexpr bool bSelected = true;
		SkeletonTreeView->SetItemSelection(SkeletonTreeItems, bSelected, InSelectInfo);
		SkeletonTreeView->RequestScrollIntoView(SkeletonTreeItems[0]);
	}
}

void SSkeletonTree::DeselectAll()
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);
		SkeletonTreeView->ClearSelection();
	}
}

TArray<TSharedPtr<ISkeletonTreeItem>> SSkeletonTree::GetSelectedItems() const 
{ 
	return SkeletonTreeView->GetSelectedItems(); 
}

void SSkeletonTree::NotifyUser( FNotificationInfo& NotificationInfo )
{
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification( NotificationInfo );
	if ( Notification.IsValid() )
	{
		Notification->SetCompletionState( SNotificationItem::CS_Fail );
	}
}

void SSkeletonTree::RegisterNewMenu()
{
	const FName MenuName("SkeletonTree.NewMenu");
	if (UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

	{
		FToolMenuSection& CreateSection = Menu->AddSection("CreateNew", LOCTEXT("SkeletonCreateNew", "Create"));

		CreateSection.AddDynamicEntry("SocketAndVirtualBone", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			USkeletonTreeMenuContext* Context = InSection.FindContext<USkeletonTreeMenuContext>();
			if (!Context)
			{
				return;
			}

			TSharedPtr<SSkeletonTree> SkeletonTree = Context->SkeletonTree.Pin();
			if (!SkeletonTree.IsValid() || !SkeletonTree->ShouldAllowSkeletonOperations())
			{
				return;
			}

			const FSkeletonTreeCommands& Actions = FSkeletonTreeCommands::Get();
			InSection.AddMenuEntry(Actions.AddSocket);
			InSection.AddSubMenu(
				"VirtualBones",
				LOCTEXT("AddVirtualBone", "Add Virtual Bone"),
				LOCTEXT("AddVirtualBone_ToolTip", "Adds a virtual bone to the skeleton."),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
					{
						USkeletonTreeMenuContext* MenuContext = InMenu->Context.FindContext<USkeletonTreeMenuContext>();
						if (MenuContext == nullptr)
						{
							return;
						}

						TSharedPtr<SSkeletonTree> InnerSkeletonTree = MenuContext->SkeletonTree.Pin();
						if (!InnerSkeletonTree.IsValid())
						{
							return;
						}

						TSharedRef<SBoneTreeMenu> MenuContent = SSkeletonTree::CreateVirtualBoneMenu(InnerSkeletonTree.Get());
						FToolMenuEntry WidgetEntry = FToolMenuEntry::InitWidget("VirtualBones", MenuContent, FText());
						InMenu->AddMenuEntry(NAME_None, WidgetEntry);
					}));
		}));
	}

	{
		FToolMenuSection& BlendSection = Menu->AddSection("Blend", LOCTEXT("SkeletonBlend", "Blend"));
		BlendSection.AddDynamicEntry("BlendProfileEntries", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			USkeletonTreeMenuContext* Context = InSection.FindContext<USkeletonTreeMenuContext>();
			if (!Context)
			{
				return;
			}

			TSharedPtr<SSkeletonTree> SkeletonTree = Context->SkeletonTree.Pin();
			if (!SkeletonTree.IsValid() || !SkeletonTree->ShouldAllowSkeletonOperations())
			{
				return;
			}

			const FSkeletonTreeCommands& Actions = FSkeletonTreeCommands::Get();
			InSection.AddMenuEntry(Actions.CreateTimeBlendProfile);
			InSection.AddMenuEntry(Actions.CreateWeightBlendProfile);
			InSection.AddMenuEntry(Actions.CreateBlendMask);
		}));
	}
}

TSharedRef< SWidget > SSkeletonTree::CreateNewMenuWidget()
{
	FToolMenuContext MenuContext(UICommandList, Extenders);
	USkeletonTreeMenuContext* SkeletonTreeMenuContext = NewObject<USkeletonTreeMenuContext>();
	SkeletonTreeMenuContext->SkeletonTree = SharedThis(this);
	MenuContext.AddObject(SkeletonTreeMenuContext);
	
	return UToolMenus::Get()->GenerateWidget("SkeletonTree.NewMenu", MenuContext); 
}

void SSkeletonTree::RegisterFilterMenu()
{
	const FName MenuName("SkeletonTree.FilterMenu");
	if (UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);

	const FSkeletonTreeCommands& Actions = FSkeletonTreeCommands::Get();

	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

	{
		FToolMenuSection& BlendProfilesSection = Menu->AddSection("BlendProfiles", LOCTEXT("BlendProfilesMenuHeading", "Blend Profiles"));
		BlendProfilesSection.AddSubMenu(
			"BlendProfiles", 
			LOCTEXT("BlendProfilesSubMenu", "Blend Profiles"),
			LOCTEXT("BlendProfilesSubMenuTooltip", "Edit Blend Profiles in this Skeleton"), 
			FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&SSkeletonTree::CreateBlendProfileMenu)));
	}

	{
		FToolMenuSection& OptionsSection = Menu->AddSection("FilterOptions", LOCTEXT("OptionsMenuHeading", "Options"));
		OptionsSection.AddMenuEntry(Actions.ShowRetargeting);
		OptionsSection.AddMenuEntry(Actions.FilteringFlattensHierarchy);
		OptionsSection.AddMenuEntry(Actions.HideParentsWhenFiltering);
		OptionsSection.AddMenuEntry(Actions.ShowDebugVisualization);
		OptionsSection.AddMenuEntry(Actions.ShowBoneIndex);
	}

	{
		FToolMenuSection& BonesSection = Menu->AddSection("FilterBones", LOCTEXT("BonesMenuHeading", "Bones"));
		BonesSection.AddMenuEntry(Actions.ShowAllBones);
		BonesSection.AddMenuEntry(Actions.ShowMeshBones);
		BonesSection.AddMenuEntry(Actions.ShowLODBones);
		BonesSection.AddMenuEntry(Actions.ShowWeightedBones);
		BonesSection.AddMenuEntry(Actions.HideBones);
	}

	{
		FToolMenuSection& BonesSection = Menu->AddSection("FilterSockets", LOCTEXT("SocketsMenuHeading", "Sockets"));
		BonesSection.AddMenuEntry(Actions.ShowActiveSockets);
		BonesSection.AddMenuEntry(Actions.ShowMeshSockets);
		BonesSection.AddMenuEntry(Actions.ShowSkeletonSockets);
		BonesSection.AddMenuEntry(Actions.ShowAllSockets);
		BonesSection.AddMenuEntry(Actions.HideSockets);
	}
}

TSharedRef< SWidget > SSkeletonTree::CreateFilterMenuWidget()
{
	FToolMenuContext MenuContext(UICommandList, Extenders);
	USkeletonTreeMenuContext* SkeletonTreeMenuContext = NewObject<USkeletonTreeMenuContext>();
	SkeletonTreeMenuContext->SkeletonTree = SharedThis(this);
	MenuContext.AddObject(SkeletonTreeMenuContext);

	return UToolMenus::Get()->GenerateWidget("SkeletonTree.FilterMenu", MenuContext);
}

void SSkeletonTree::SetBoneFilter( EBoneFilter InBoneFilter )
{
	check( InBoneFilter < EBoneFilter::Count );
	BoneFilter = InBoneFilter;

	ApplyFilter();
}

bool SSkeletonTree::IsBoneFilter( EBoneFilter InBoneFilter ) const
{
	return BoneFilter == InBoneFilter;
}

void SSkeletonTree::SetSocketFilter( ESocketFilter InSocketFilter )
{
	check( InSocketFilter < ESocketFilter::Count );
	SocketFilter = InSocketFilter;

	SetPreviewComponentSocketFilter();

	ApplyFilter();
}

void SSkeletonTree::SetPreviewComponentSocketFilter() const
{
	// Set the socket filter in the debug skeletal mesh component so the viewport can share the filter settings
	if (GetPreviewScene().IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

		bool bAllOrActive = (SocketFilter == ESocketFilter::All || SocketFilter == ESocketFilter::Active);

		if (PreviewComponent)
		{
			PreviewComponent->bMeshSocketsVisible = bAllOrActive || SocketFilter == ESocketFilter::Mesh;
			PreviewComponent->bSkeletonSocketsVisible = bAllOrActive || SocketFilter == ESocketFilter::Skeleton;
		}
	}
}

bool SSkeletonTree::IsSocketFilter( ESocketFilter InSocketFilter ) const
{
	return SocketFilter == InSocketFilter;
}

FText SSkeletonTree::GetFilterMenuTooltip() const
{
	TArray<FText> FilterLabels;

	if(Builder->IsShowingBones())	
	{
		switch ( BoneFilter )
		{
			case EBoneFilter::All:
			FilterLabels.Add(LOCTEXT( "BoneFilterMenuAll", "Bones" ));
			break;

			case EBoneFilter::Mesh:
			FilterLabels.Add(LOCTEXT( "BoneFilterMenuMesh", "Mesh Bones" ));
			break;

			case EBoneFilter::LOD:
			FilterLabels.Add(LOCTEXT("BoneFilterMenuLOD", "LOD Bones"));
			break;

			case EBoneFilter::Weighted:
			FilterLabels.Add(LOCTEXT( "BoneFilterMenuWeighted", "Weighted Bones" ));
			break;

			case EBoneFilter::None:
			break;

			default:
			// Unknown mode
			check(false);
			break;
		}
	}

	if(Builder->IsShowingSockets())
	{
		switch (SocketFilter)
		{
			case ESocketFilter::Active:
			FilterLabels.Add(LOCTEXT("SocketFilterMenuActive", "Active Sockets"));
			break;

			case ESocketFilter::Mesh:
			FilterLabels.Add(LOCTEXT("SocketFilterMenuMesh", "Mesh Sockets"));
			break;

			case ESocketFilter::Skeleton:
			FilterLabels.Add(LOCTEXT("SocketFilterMenuSkeleton", "Skeleton Sockets"));
			break;

			case ESocketFilter::All:
			FilterLabels.Add(LOCTEXT("SocketFilterMenuAll", "All Sockets"));
			break;

			case ESocketFilter::None:
			break;

			default:
			// Unknown mode
			check(false);
			break;
		}
	}

	OnGetFilterText.ExecuteIfBound(FilterLabels);

	FText Label;
	if(FilterLabels.Num() > 0)
	{
		Label = FText::Format(LOCTEXT("FilterMenuLabelFormatStart", "Showing: {0}"), FilterLabels[0]);
		for(int32 LabelIndex = 1; LabelIndex < FilterLabels.Num(); ++LabelIndex)
		{
			Label = FText::Format(LOCTEXT("FilterMenuLabelFormat", "{0}, {1}"), Label, FilterLabels[LabelIndex]);
		}
		Label = FText::Format(LOCTEXT("FilterMenuLabelFormatEnd", "{0}\nShift-clicking on items will 'pin' them to the skeleton tree."), Label);
	}
	else
	{
		Label = LOCTEXT("ShowingNoneLabel", "Filters.\nShift-clicking on items will 'pin' them to the skeleton tree.");
	}
	
	return Label;
}

bool SSkeletonTree::IsAddingSocketsAllowed() const
{
	if ( SocketFilter == ESocketFilter::Skeleton ||
		SocketFilter == ESocketFilter::Active ||
		SocketFilter == ESocketFilter::All )
	{
		TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
		FSkeletonTreeSelection TreeSelection(SelectedItems);
		return TreeSelection.IsSingleOfTypesSelected<FSkeletonTreeBoneItem, FSkeletonTreeVirtualBoneItem>();
	}

	return false;
}

FReply SSkeletonTree::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( UICommandList->ProcessCommandBindings( InKeyEvent ) )
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SSkeletonTree::CanDeleteSelectedRows() const
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);
	return (TreeSelection.HasSelectedOfType<FSkeletonTreeAttachedAssetItem>() || TreeSelection.HasSelectedOfType<FSkeletonTreeSocketItem>() || TreeSelection.HasSelectedOfType<FSkeletonTreeVirtualBoneItem>());
}

void SSkeletonTree::OnDeleteSelectedRows()
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
	FSkeletonTreeSelection TreeSelection(SelectedItems);

	if(TreeSelection.HasSelectedOfType<FSkeletonTreeAttachedAssetItem>() || TreeSelection.HasSelectedOfType<FSkeletonTreeSocketItem>() || TreeSelection.HasSelectedOfType<FSkeletonTreeVirtualBoneItem>())
	{
		FScopedTransaction Transaction( LOCTEXT( "SkeletonTreeDeleteSelected", "Delete selected sockets/meshes/bones from skeleton tree" ) );

		DeleteAttachedAssets( TreeSelection.GetSelectedItems<FSkeletonTreeAttachedAssetItem>() );
		DeleteSockets( TreeSelection.GetSelectedItems<FSkeletonTreeSocketItem>() );
		DeleteVirtualBones( TreeSelection.GetSelectedItems<FSkeletonTreeVirtualBoneItem>() );

		CreateFromSkeleton();
	}
}

void SSkeletonTree::DeleteAttachedAssets(const TArray<TSharedPtr<FSkeletonTreeAttachedAssetItem>>& InDisplayedAttachedAssetInfos)
{
	DeselectAll();

	TArray<FPreviewAttachedObjectPair> AttachedObjects;
	for(const TSharedPtr<FSkeletonTreeAttachedAssetItem>& AttachedAssetInfo : InDisplayedAttachedAssetInfos)
	{
		FPreviewAttachedObjectPair Pair;
		Pair.SetAttachedObject(AttachedAssetInfo->GetAsset());
		Pair.AttachedTo = AttachedAssetInfo->GetParentName();

		AttachedObjects.Add(Pair);
	}

	GetEditableSkeletonInternal()->HandleDeleteAttachedAssets(AttachedObjects, GetPreviewScene());
}

void SSkeletonTree::DeleteSockets(const TArray<TSharedPtr<FSkeletonTreeSocketItem>>& InDisplayedSocketInfos)
{
	DeselectAll();

	TArray<FSelectedSocketInfo> SocketInfo;

	for (const TSharedPtr<FSkeletonTreeSocketItem>& DisplayedSocketInfo : InDisplayedSocketInfos)
	{
		USkeletalMeshSocket* SocketToDelete = DisplayedSocketInfo->GetSocket();
		SocketInfo.Add(FSelectedSocketInfo(SocketToDelete, DisplayedSocketInfo->GetParentType() == ESocketParentType::Skeleton));
	}

	GetEditableSkeletonInternal()->HandleDeleteSockets(SocketInfo, GetPreviewScene());
}

void SSkeletonTree::DeleteVirtualBones(const TArray<TSharedPtr<FSkeletonTreeVirtualBoneItem>>& InDisplayedVirtualBoneInfos)
{
	DeselectAll();

	TArray<FName> VirtualBoneInfo;

	for (const TSharedPtr<FSkeletonTreeVirtualBoneItem>& DisplayedVirtualBoneInfo : InDisplayedVirtualBoneInfos)
	{
		VirtualBoneInfo.Add(DisplayedVirtualBoneInfo->GetRowItemName());
	}

	GetEditableSkeletonInternal()->HandleDeleteVirtualBones(VirtualBoneInfo, GetPreviewScene());
}

void SSkeletonTree::OnChangeShowingAdvancedOptions()
{
	SkeletonTreeView->GetHeaderRow()->SetShowGeneratedColumn(ISkeletonTree::Columns::Retargeting, !IsShowingAdvancedOptions());
	HandleTreeRefresh();
}

bool SSkeletonTree::IsShowingAdvancedOptions() const
{
	return SkeletonTreeView->GetHeaderRow()->IsColumnVisible(ISkeletonTree::Columns::Retargeting);
}

void SSkeletonTree::OnChangeShowingDebugVisualizationOptions()
{
	SkeletonTreeView->GetHeaderRow()->SetShowGeneratedColumn(ISkeletonTree::Columns::DebugVisualization, !IsShowingDebugVisualizationOptions());
}

bool SSkeletonTree::IsShowingDebugVisualizationOptions() const
{
	return SkeletonTreeView->GetHeaderRow()->IsColumnVisible(ISkeletonTree::Columns::DebugVisualization);
}

UBlendProfile* SSkeletonTree::GetSelectedBlendProfile()
{
	return BlendProfilePicker->GetSelectedBlendProfile();
}

FName SSkeletonTree::GetSelectedBlendProfileName() const
{
	return BlendProfilePicker->GetSelectedBlendProfileName();
}

void SSkeletonTree::OnBlendProfileSelected(UBlendProfile* NewProfile)
{
	SkeletonTreeView->GetHeaderRow()->RefreshColumns();
	if (NewProfile != nullptr)
		SkeletonTreeView->GetHeaderRow()->SetShowGeneratedColumn(ISkeletonTree::Columns::BlendProfile);
	HandleTreeRefresh();

	// When a new blend profile is created/selected - enable edition if name != None.
	BlendProfileHeader->SetReadOnly(BlendProfilePicker->GetSelectedBlendProfileName() == NAME_None);
}

void SSkeletonTree::RecursiveSetBlendProfileScales(float InScaleToSet)
{
	UBlendProfile* SelectedBlendProfile = BlendProfilePicker->GetSelectedBlendProfile();
	if(SelectedBlendProfile)
	{
		TArray<FName> BoneNames;
		TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTreeView->GetSelectedItems();
		FSkeletonTreeSelection TreeSelection(SelectedItems);

		for(TSharedPtr<FSkeletonTreeBoneItem>& SelectedBone : TreeSelection.GetSelectedItems<FSkeletonTreeBoneItem>())
		{
			BoneNames.Add(SelectedBone->GetRowItemName());
		}

		GetEditableSkeletonInternal()->RecursiveSetBlendProfileScales(SelectedBlendProfile->GetFName(), BoneNames, InScaleToSet);

		HandleTreeRefresh();
	}
}

void SSkeletonTree::HandleTreeRefresh()
{
	SkeletonTreeView->RequestTreeRefresh();
}

void SSkeletonTree::PostRenameSocket(UObject* InAttachedObject, const FName& InOldName, const FName& InNewName)
{
	TSharedPtr<IPersonaPreviewScene> LinkedPreviewScene = GetPreviewScene();
	if (LinkedPreviewScene.IsValid())
	{
		LinkedPreviewScene->RemoveAttachedObjectFromPreviewComponent(InAttachedObject, InOldName);
		LinkedPreviewScene->AttachObjectToPreviewComponent(InAttachedObject, InNewName);
	}
}

void SSkeletonTree::PostDuplicateSocket(UObject* InAttachedObject, const FName& InSocketName)
{
	TSharedPtr<IPersonaPreviewScene> LinkedPreviewScene = GetPreviewScene();
	if (LinkedPreviewScene.IsValid())
	{
		LinkedPreviewScene->AttachObjectToPreviewComponent(InAttachedObject, InSocketName);
	}

	CreateFromSkeleton();
}

void SSkeletonTree::PostSetSocketParent()
{
	CreateFromSkeleton();
}

void RecursiveSetLODChange(UDebugSkelMeshComponent* PreviewComponent, TSharedPtr<ISkeletonTreeItem> TreeRow)
{
	if (TreeRow->IsOfType<FSkeletonTreeBoneItem>())
	{
		StaticCastSharedPtr<FSkeletonTreeBoneItem>(TreeRow)->CacheLODChange(PreviewComponent);
	}
	
	for (auto& Child : TreeRow->GetChildren())
	{
		RecursiveSetLODChange(PreviewComponent, Child);
	}
}

void SSkeletonTree::OnLODSwitched()
{
	if (GetPreviewScene().IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

		if (PreviewComponent)
		{
			LastCachedLODForPreviewMeshComponent = PreviewComponent->GetPredictedLODLevel();

			if (BoneFilter == EBoneFilter::Weighted || BoneFilter == EBoneFilter::LOD)
			{
				CreateFromSkeleton();
			}
			else
			{
				for (TSharedPtr<ISkeletonTreeItem>& Item : Items)
				{
					RecursiveSetLODChange(PreviewComponent, Item);
				}
			}
		}
	}
}

void SSkeletonTree::OnPreviewMeshChanged(USkeletalMesh* InOldSkeletalMesh, USkeletalMesh* InNewSkeletalMesh)
{
	if (InOldSkeletalMesh != InNewSkeletalMesh || InNewSkeletalMesh == nullptr)
	{
		DeselectAll();
	}
}

void SSkeletonTree::SelectItemsBy(TFunctionRef<bool(const TSharedRef<ISkeletonTreeItem>&, bool&)> Predicate, const ESelectInfo::Type SelectInfo) const
{
	TArray<TPair<TSharedPtr<ISkeletonTreeItem>, bool>> ItemsToSelect;
	TSharedPtr<ISkeletonTreeItem> ScrollItem = nullptr;
	for (const TSharedPtr<ISkeletonTreeItem>& Item : LinearItems)
	{
		if(Item->GetFilterResult() != ESkeletonTreeFilterResult::Hidden)
		{
			bool bExpand = false;
			if (Predicate(Item.ToSharedRef(), bExpand))
			{
				ItemsToSelect.Emplace(Item, bExpand);
				ScrollItem = Item;
			}
		}
	}

	if(ItemsToSelect.Num() > 0)
	{
		SkeletonTreeView->ClearSelection();

		for (const TPair<TSharedPtr<ISkeletonTreeItem>, bool>& ItemPair : ItemsToSelect)
		{
			TSharedPtr<ISkeletonTreeItem> Item = ItemPair.Key;

			SkeletonTreeView->SetItemSelection(Item, true, SelectInfo);
			if (ItemPair.Value)
			{
				if(Item->GetChildren().Num() == 0)
				{
					// leaf nodes expand their parent
					TSharedPtr<ISkeletonTreeItem> ParentItem = Item->GetParent();
					if(ParentItem.IsValid())
					{
						SkeletonTreeView->SetItemExpansion(ParentItem, true);
					}
				}
				else
				{
					SkeletonTreeView->SetItemExpansion(Item, true);
				}
			}
		}
	}

	if (ScrollItem.IsValid())
	{
		SkeletonTreeView->RequestScrollIntoView(ScrollItem);
	}
}

void SSkeletonTree::DuplicateAndSelectSocket(const FSelectedSocketInfo& SocketInfoToDuplicate, const FName& NewParentBoneName /*= FName()*/)
{
	USkeletalMesh* SkeletalMesh = GetPreviewScene().IsValid() ? ToRawPtr(GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;
	USkeletalMeshSocket* NewSocket = GetEditableSkeleton()->DuplicateSocket(SocketInfoToDuplicate, NewParentBoneName, SkeletalMesh);

	if (GetPreviewScene().IsValid())
	{
		GetPreviewScene()->SetSelectedSocket(FSelectedSocketInfo(NewSocket, SocketInfoToDuplicate.bSocketIsOnSkeleton));
	}

	CreateFromSkeleton();

	FSelectedSocketInfo NewSocketInfo(NewSocket, SocketInfoToDuplicate.bSocketIsOnSkeleton);
	SetSelectedSocket(NewSocketInfo);
}

void SSkeletonTree::ResetBoneTransforms(const bool bSelectedOnly)
{
	OnResetBoneTransforms(bSelectedOnly);
}

void SSkeletonTree::HandleFocusCamera()
{
	if (GetPreviewScene().IsValid())
	{
		GetPreviewScene()->FocusViews();
	}

	if(!SkeletonTreeView->GetSelectedItems().IsEmpty())
	{
		TSharedPtr<class ISkeletonTreeItem> SelectedRow = SkeletonTreeView->GetSelectedItems()[0]; 
		ExpandTreeOnSelection(SelectedRow);
		SkeletonTreeView->RequestScrollIntoView(SelectedRow);
	}
}

ESkeletonTreeFilterResult SSkeletonTree::HandleFilterSkeletonTreeItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem)
{
	ESkeletonTreeFilterResult Result = ESkeletonTreeFilterResult::Shown;

	if(InItem->IsOfType<FSkeletonTreeBoneItem>() || InItem->IsOfType<FSkeletonTreeSocketItem>() || InItem->IsOfType<FSkeletonTreeAttachedAssetItem>() || InItem->IsOfType<FSkeletonTreeVirtualBoneItem>())
	{

		if (InArgs.TextFilter.IsValid())
		{
			if (InArgs.TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(InItem->GetRowItemName().ToString())))
			{
				Result = ESkeletonTreeFilterResult::ShownHighlighted;
			}
			else
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}


		if (InItem->IsOfType<FSkeletonTreeBoneItem>())
		{
			TSharedPtr<FSkeletonTreeBoneItem> BoneItem = StaticCastSharedPtr<FSkeletonTreeBoneItem>(InItem);

			if (BoneFilter == EBoneFilter::None)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
			else if (GetPreviewScene().IsValid())
			{
				UDebugSkelMeshComponent* PreviewMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
				if (PreviewMeshComponent)
				{
					const int32 BoneMeshIndex = PreviewMeshComponent->GetBoneIndex(BoneItem->GetRowItemName());

					// Remove non-mesh bones if we're filtering
					if ((BoneFilter == EBoneFilter::Mesh || BoneFilter == EBoneFilter::Weighted || BoneFilter == EBoneFilter::LOD) &&
						BoneMeshIndex == INDEX_NONE)
					{
						Result = ESkeletonTreeFilterResult::Hidden;
					}

					// Remove non-vertex-weighted bones if we're filtering
					else if (BoneFilter == EBoneFilter::Weighted && !BoneItem->IsBoneWeighted(BoneMeshIndex, PreviewMeshComponent))
					{
						Result = ESkeletonTreeFilterResult::Hidden;
					}

					// Remove non-vertex-weighted bones if we're filtering
					else if (BoneFilter == EBoneFilter::LOD && !BoneItem->IsBoneRequired(BoneMeshIndex, PreviewMeshComponent))
					{
						Result = ESkeletonTreeFilterResult::Hidden;
					}
				}
			}
		}
		else if (InItem->IsOfType<FSkeletonTreeSocketItem>())
		{
			TSharedPtr<FSkeletonTreeSocketItem> SocketItem = StaticCastSharedPtr<FSkeletonTreeSocketItem>(InItem);

			if (SocketFilter == ESocketFilter::None)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}

			// Remove non-mesh sockets if we're filtering
			else if ((SocketFilter == ESocketFilter::Mesh || SocketFilter == ESocketFilter::None) && SocketItem->GetParentType() == ESocketParentType::Skeleton)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}

			// Remove non-skeleton sockets if we're filtering
			else if ((SocketFilter == ESocketFilter::Skeleton || SocketFilter == ESocketFilter::None) && SocketItem->GetParentType() == ESocketParentType::Mesh)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}

			else if (SocketFilter == ESocketFilter::Active && SocketItem->GetParentType() == ESocketParentType::Skeleton && SocketItem->IsSocketCustomized())
			{
				// Don't add the skeleton socket if it's already added for the mesh
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}
	}

	return Result;
}

void SSkeletonTree::HandleSelectedBonesChanged(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo)
{
	SetSelectedBones(InBoneNames, InSelectInfo);
}

void SSkeletonTree::HandleSelectedSocketChanged(const FSelectedSocketInfo& InSocketInfo)
{
	SetSelectedSocket(InSocketInfo);
}

void SSkeletonTree::HandleDeselectAll()
{
	DeselectAll();
}

#undef LOCTEXT_NAMESPACE
