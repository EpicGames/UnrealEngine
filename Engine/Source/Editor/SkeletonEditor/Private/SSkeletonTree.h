// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Attribute.h"
#include "Animation/Skeleton.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "EditableSkeleton.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/SkeletalMeshSocket.h"
#include "ISkeletonTree.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Preferences/PersonaOptions.h"
#include "ISkeletonTreeBuilder.h"
#include "ISkeletonTreeItem.h"
#include "SkeletonTreeBuilder.h"
#include "Widgets/Input/SSearchBox.h"
#include "EditorUndoClient.h"
#include "UObject/GCObject.h"

class FMenuBuilder;
class FSkeletonTreeAttachedAssetItem;
class FSkeletonTreeBoneItem;
class FSkeletonTreeSocketItem;
class FSkeletonTreeVirtualBoneItem;
class FTextFilterExpressionEvaluator;
class FUICommandList_Pinnable;
class IPersonaPreviewScene;
class SBlendProfilePicker;
class SComboButton;
template <typename ItemType> class SSkeletonTreeView;
class UBlendProfile;
class UToolMenu;
struct FNotificationInfo;
class IPinnedCommandList;
class FPackageReloadedEvent;
enum class EPackageReloadPhase : uint8;
enum class EBlendProfileMode : uint8;

//////////////////////////////////////////////////////////////////////////
// SSkeletonTree

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class SSkeletonTree : public ISkeletonTree, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SSkeletonTree )
		: _IsEditable(true)
		{}

	SLATE_ATTRIBUTE( bool, IsEditable )

	SLATE_ARGUMENT( TSharedPtr<class ISkeletonTreeBuilder>, Builder )

	SLATE_END_ARGS()
public:
	SSkeletonTree()
		: bSelecting(false)
	{}

	~SSkeletonTree();

	void Construct(const FArguments& InArgs, const TSharedRef<FEditableSkeleton>& InEditableSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs);

	/** ISkeletonTree interface */
	virtual void Refresh() override;
	virtual void RefreshFilter() override;
	virtual TSharedRef<class IEditableSkeleton> GetEditableSkeleton() const override { return EditableSkeleton.Pin().ToSharedRef(); }
	virtual TSharedPtr<class IPersonaPreviewScene> GetPreviewScene() const override { return PreviewScene.Pin(); }
	virtual void SetSkeletalMesh(USkeletalMesh* NewSkeletalMesh) override;
	virtual void SetSelectedSocket(const struct FSelectedSocketInfo& InSocketInfo) override;
	virtual void SetSelectedBone(const FName& InBoneName, ESelectInfo::Type InSelectInfo) override;
	virtual void SetSelectedBones(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo) override;
	virtual void DeselectAll() override;
	virtual TArray<TSharedPtr<ISkeletonTreeItem>> GetSelectedItems() const override;
	virtual void SelectItemsBy(TFunctionRef<bool(const TSharedRef<ISkeletonTreeItem>&, bool&)> Predicate, const ESelectInfo::Type SelectInfo) const override;
	virtual void DuplicateAndSelectSocket(const FSelectedSocketInfo& SocketInfoToDuplicate, const FName& NewParentBoneName = FName()) override;
	virtual void ResetBoneTransforms(const bool bSelectedOnly) override;

	virtual FDelegateHandle RegisterOnSelectionChanged(const FOnSkeletonTreeSelectionChanged& Delegate) override
	{
		return OnSelectionChangedMulticast.Add(Delegate);
	}

	virtual void UnregisterOnSelectionChanged(FDelegateHandle DelegateHandle) override
	{
		OnSelectionChangedMulticast.Remove(DelegateHandle);
	}

	virtual UBlendProfile* GetSelectedBlendProfile() override;
	virtual void AttachAssets(const TSharedRef<ISkeletonTreeItem>& TargetItem, const TArray<FAssetData>& AssetData) override;
	virtual TSharedPtr<SWidget> GetSearchWidget() const override { return NameFilterBox; }
	virtual TSharedPtr<IPinnedCommandList> GetPinnedCommandList() const override { return PinnedCommands; }

	bool ShouldAllowSkeletonOperations() const { return bAllowSkeletonOperations; }
	bool ShouldAllowMeshOperations() const { return bAllowMeshOperations; }

	/** FSelfRegisteringEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** Creates the tree control and then populates */
	void CreateTreeColumns();

	/** Function to build the skeleton tree widgets from the source skeleton tree */
	void CreateFromSkeleton();

	/** Apply filtering to the tree */
	void ApplyFilter();

	/** Set the initial expansion state of the tree items */
	void SetInitialExpansionState();

	/** Utility function to print notifications to the user */
	void NotifyUser( FNotificationInfo& NotificationInfo );

	/** Callback when an item is scrolled into view, handling calls to rename items */
	void OnItemScrolledIntoView( TSharedPtr<ISkeletonTreeItem> InItem, const TSharedPtr<ITableRow>& InWidget);

	/** Callback for when the user double-clicks on an item in the tree */
	void OnTreeDoubleClick( TSharedPtr<ISkeletonTreeItem> InItem );

	/** Handle recursive expansion/contraction of the tree */
	void SetTreeItemExpansionRecursive(TSharedPtr< ISkeletonTreeItem > TreeItem, bool bInExpansionState) const;

	/** Set Bone Translation Retargeting Mode for bone selection, and their children. */
	void SetBoneTranslationRetargetingModeRecursive(EBoneTranslationRetargetingMode::Type NewRetargetingMode);
	
	enum class EBoneReductionSelectionMode : uint8
	{
		SelectedBones,
		ChildrenOfSelectedBones,
		AllUnusedBones
	};
	void RemoveFromLOD(int32 LODIndex, EBoneReductionSelectionMode SelectionMode, bool bIncludeBelowLODs);

	/** Remove the selected bones from LOD of LODIndex when using simplygon **/
	UE_DEPRECATED(5.8, "RemoveFromLOD(int32, bool, bool) is deprecated. Use RemoveFromLOD(int32, bool, ERemoveFromLODMode).")
	inline void RemoveFromLOD(int32 LODIndex, bool bIncludeSelected, bool bIncludeBelowLODs)
	{
		const EBoneReductionSelectionMode SelectionMode = bIncludeSelected ? EBoneReductionSelectionMode::SelectedBones : EBoneReductionSelectionMode::ChildrenOfSelectedBones;
		RemoveFromLOD(LODIndex, SelectionMode, bIncludeBelowLODs);
	}

	/** Called when the preview mesh is changed - simply rebuilds the skeleton tree for the new mesh */
	void OnLODSwitched();

	/** Called when the preview mesh is changed */
	void OnPreviewMeshChanged(USkeletalMesh* InOldSkeletalMesh, USkeletalMesh* InNewSkeletalMesh);

	/** Get the name of the currently selected blend profile */
	FName GetSelectedBlendProfileName() const;

	/** Delegate handler for when the tree needs refreshing */
	void HandleTreeRefresh();

	/** Get a shared reference to the skeleton tree that owns us */
	TSharedRef<FEditableSkeleton> GetEditableSkeletonInternal() const { return EditableSkeleton.Pin().ToSharedRef(); }

	/** Update preview scene and tree after a socket rename */
	void PostRenameSocket(UObject* InAttachedObject, const FName& InOldName, const FName& InNewName);

	/** Update preview scene and tree after a socket duplication */
	void PostDuplicateSocket(UObject* InAttachedObject, const FName& InSocketName);

	/** Update tree after a socket changes parent */
	void PostSetSocketParent();

private:
	/** Binds the commands in FSkeletonTreeCommands to functions in this class */
	void BindCommands();

	/** Create a widget for an entry in the tree from an info */
	TSharedRef<ITableRow> MakeTreeRowWidget(TSharedPtr<ISkeletonTreeItem> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get all children for a given entry in the list */
	void GetFilteredChildren(TSharedPtr<ISkeletonTreeItem> InInfo, TArray< TSharedPtr<ISkeletonTreeItem> >& OutChildren);

	/** Called to display context menu when right clicking on the widget */
	TSharedPtr< SWidget > CreateContextMenu();

	/** Called to display the add new menu */
	TSharedRef< SWidget > CreateNewMenuWidget();

	/** Called to create the add new menu */
	void RegisterNewMenu();

	/** Called to display the filter menu */
	TSharedRef< SWidget > CreateFilterMenuWidget();

	/** Called to create the filter menu */
	void RegisterFilterMenu();

	/** Function to copy selected bone name to the clipboard */
	void OnCopyBoneNames();

	/** Function to reset the transforms of selected bones */
	void OnResetBoneTransforms(const bool bSelectedOnly = true);

	/** Function to copy selected sockets to the clipboard */
	void OnCopySockets() const;

	/** Whether we can copy sockets */
	bool CanCopySockets() const;

	/** Function to serialize a single socket to a string */
	FString SerializeSocketToString( class USkeletalMeshSocket* Socket, ESocketParentType ParentType ) const;

	/** Function to paste selected sockets from the clipboard */
	void OnPasteSockets(bool bPasteToSelectedBone);

	/** Whether we can paste sockets */
	bool CanPasteSockets() const;

	/** Function to add a socket to the selected bone (skeleton, not mesh) */
	void OnAddSocket();

	/** Function to check if it is possible to rename the selected item */
	bool CanRenameSelected() const;

	/** Function to start renaming the selected item */
	void OnRenameSelected();

	/** Function to customize a socket - this essentially copies a socket from the skeleton to the mesh */
	void OnCustomizeSocket();

	/** Function to promote a socket - this essentially copies a socket from the mesh to the skeleton */
	void OnPromoteSocket();

	/** Create sub menu to allow users to pick a target bone for the new space switching bone(s) */
	static TSharedRef<class SBoneTreeMenu> CreateVirtualBoneMenu(SSkeletonTree* InSkeletonTree);

	/** Handler for user picking a target bone */
	void OnVirtualTargetBonePicked(FName TargetBoneName, TArray<TSharedPtr<class ISkeletonTreeItem>> SourceBones);

	/** Create content picker sub menu to allow users to pick an asset to attach */
	void FillAttachAssetSubmenu(FMenuBuilder& MenuBuilder, const TSharedPtr<class ISkeletonTreeItem> TargetItem);

	/** Helper function for asset picker that handles users choice */
	void OnAssetSelectedFromPicker(const FAssetData& AssetData, const TSharedPtr<class ISkeletonTreeItem> TargetItem);

	/** Context menu function to remove all attached assets */
	void OnRemoveAllAssets();

	/** Context menu function to control enabled/disabled status of remove all assets menu item */
	bool CanRemoveAllAssets() const;

	/** Functions to copy sockets from the skeleton to the mesh */
	void OnCopySocketToMesh() {};

	/** Callback function to be called when selection changes to check if the next item is selectable or navigable. */
	bool OnIsSelectableOrNavigable(TSharedPtr<class ISkeletonTreeItem> InItem) const;

	/** Callback function to be called when selection changes in the tree view widget. */
	void OnSelectionChanged(TSharedPtr<class ISkeletonTreeItem> Selection, ESelectInfo::Type SelectInfo);

	/** Filters the SListView when the user changes the search text box (NameFilterBox)	*/
	void OnFilterTextChanged( const FText& SearchText );

	/** Sets which types of bone to show */
	void SetBoneFilter(EBoneFilter InBoneFilter );

	/** Queries the bone filter */
	bool IsBoneFilter(EBoneFilter InBoneFilter ) const;

	/** Sets which types of socket to show */
	void SetSocketFilter(ESocketFilter InSocketFilter );

	/** Queries the bone filter */
	bool IsSocketFilter(ESocketFilter InSocketFilter ) const;

	/** Returns the current text for the filter button tooltip - "All", "Mesh" or "Weighted" etc. */
	FText GetFilterMenuTooltip() const;

	/** We can only add sockets in Active, Skeleton or All mode (otherwise they just disappear) */
	bool IsAddingSocketsAllowed() const;

	/** Handler for "Show Retargeting Options" check box IsChecked functionality */
	bool IsShowingAdvancedOptions() const;

	/**  Handler for when we change the "Show Retargeting Options" check box */
	void OnChangeShowingAdvancedOptions();

	/** Handler for "Show Debug Visualization Options" check box IsChecked functionality */
	bool IsShowingDebugVisualizationOptions() const;

	/**  Handler for when we change the "Show Debug Visualization Options" check box */
	void OnChangeShowingDebugVisualizationOptions();

	/** This replicates the socket filter to the previewcomponent so that the viewport can use the same settings */
	void SetPreviewComponentSocketFilter() const;

	/** Override OnKeyDown */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent );

	/** Check whether we can delete all the selected sockets/assets */
	bool CanDeleteSelectedRows() const;

	/** Function to delete all the selected sockets/assets */
	void OnDeleteSelectedRows();

	/** Function to remove attached assets from the skeleton/mesh */
	void DeleteAttachedAssets(const TArray<TSharedPtr<class FSkeletonTreeAttachedAssetItem>>& InDisplayedAttachedAssetInfos );

	/** Function to remove sockets from the skeleton/mesh */
	void DeleteSockets(const TArray<TSharedPtr<class FSkeletonTreeSocketItem>>& InDisplayedSocketInfos );

	/** Function to remove virtual bones from the skeleton/mesh */
	void DeleteVirtualBones(const TArray<TSharedPtr<class FSkeletonTreeVirtualBoneItem>>& InDisplayedVirtualBonestInfos);

	/** Called when the user selects a blend profile to edit */
	void OnBlendProfileSelected(UBlendProfile* NewProfile);

	/** Sets the blend scale for the selected bones and all of their children */
	void RecursiveSetBlendProfileScales(float InScaleToSet);

	/** Submenu creator handler for the given skeleton */
	static void CreateMenuForBoneReduction(FMenuBuilder& MenuBuilder, SSkeletonTree* SkeletonTree, int32 LODIndex, bool bIncludeSelected);

	/** Handle focusing the camera on the current selection */
	void HandleFocusCamera();

	/** Handle framing the selected item in the tree */
	void HandleFrameSelection();

	/** Handle filtering the tree  */
	ESkeletonTreeFilterResult HandleFilterSkeletonTreeItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem);

	// Called when bone tree queries reference skeleton
	const FReferenceSkeleton& OnGetReferenceSkeleton() const
	{
		return GetEditableSkeletonInternal()->GetSkeleton().GetReferenceSkeleton();
	}

	/** Handle bone selection changing externally */
	void HandleSelectedBoneChanged(const FName& InBoneName, ESelectInfo::Type InSelectInfo);
	void HandleSelectedBonesChanged(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo);

	/** Handle socket selection changing externally */
	void HandleSelectedSocketChanged(const FSelectedSocketInfo& InSocketInfo);

	/** Handle external deselection event */
	void HandleDeselectAll();

	/** Handle package reloading (might be our skeleton) */
	void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/** Creates a new Blend Profile */
	void OnCreateBlendProfile(const EBlendProfileMode InMode);

	/** Removes the active Blend Profile */
	void OnDeleteCurrentBlendProfile();

	/** Removes the active Blend Profile */
	void OnRenameBlendProfile();

	/** Register Blend Profile Menu */
	void RegisterBlendProfileMenu();

	/** Create Blend Profile Menu */
	static void CreateBlendProfileMenu(UToolMenu* InMenu);

	TSharedRef<SWidget> GetBlendProfileColumnMenuContent();

	void ExpandTreeOnSelection(TSharedPtr<ISkeletonTreeItem> RowToExpand, bool bForce = false);

private:
	/** Pointer back to the skeleton tree that owns us */
	TWeakPtr<FEditableSkeleton> EditableSkeleton;

	/** Link to a preview scene */
	TWeakPtr<IPersonaPreviewScene> PreviewScene;

	/** SSearchBox to filter the tree */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** The blend profile picker displaying the selected profile */
	TSharedPtr<SBlendProfilePicker> BlendProfilePicker;

	/** Blend Profile Header Label.  Also used to name new blend profiles */
	TSharedPtr<SInlineEditableTextBlock> BlendProfileHeader;

	/** Widget user to hold the skeleton tree */
	TSharedPtr<SOverlay> TreeHolder;

	/** Widget used to display the skeleton hierarchy */
	TSharedPtr<SSkeletonTreeView<TSharedPtr<class ISkeletonTreeItem>>> SkeletonTreeView;

	/** A tree of unfiltered items */
	TArray<TSharedPtr<class ISkeletonTreeItem>> Items;

	/** A "mirror" of the tree as a flat array for easier searching */
	TArray<TSharedPtr<class ISkeletonTreeItem>> LinearItems;

	/** Filtered view of the skeleton tree. This is what is actually used in the tree widget */
	TArray<TSharedPtr<class ISkeletonTreeItem>> FilteredItems;

	/** Is this view editable */
	TAttribute<bool> IsEditable;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList_Pinnable> UICommandList;

	/** Current type of blend profile no create. We shouldn't need to hold state for this, but blend profile creation is tied to its header text committed*/
	EBlendProfileMode NewBlendProfileMode;

	/** Whether we are creating or renaming a BlendProfile, so the EditBox know what to do */
	bool bIsCreateNewBlendProfile = false;

	/** Current type of bones to show */
	EBoneFilter BoneFilter;

	/** Current type of sockets to show */
	ESocketFilter SocketFilter;

	/** Points to an item that is being requested to be renamed */
	TSharedPtr<ISkeletonTreeItem> DeferredRenameRequest;

	/** Last Cached Preview Mesh Component LOD */
	int32 LastCachedLODForPreviewMeshComponent;

	/** Delegate for when an item is selected */
	FOnSkeletonTreeSelectionChangedMulticast OnSelectionChangedMulticast;

	/** Selection recursion guard flag */
	bool bSelecting;

	/** Hold onto the filter combo button to set its foreground color */
	TSharedPtr<SComboButton> FilterComboButton;

	/** Add virtual bones to the skeleton tree */
	void AddVirtualBones(const TArray<FVirtualBone>& VirtualBones);

	/** Checks if the named profile is the currently selected/active one */
	bool IsBlendProfileSelected(FName ProfileName) const;

	/** The builder we use to construct the tree */
	TSharedPtr<class ISkeletonTreeBuilder> Builder;

	/** Compiled filter search terms. */
	TSharedPtr<class FTextFilterExpressionEvaluator> TextFilterPtr;

	/** Whether to allow operations that modify the mesh */
	bool bAllowMeshOperations;

	/** Whether to allow operations that modify the mesh */
	bool bAllowSkeletonOperations;

	/** Whether to show the filter option to allow filtering of debug draw elements in the viewport. */
	bool bShowDebugVisualizationOptions;

	/** Wether children can remain selected while their parent is collapsed in the tree */
	bool bAllowInvisibleItemSelection = false;

	/** Extenders for menus */
	TSharedPtr<FExtender> Extenders;

	/** Delegate that allows custom filtering text to be shown on the filter button */
	FOnGetFilterText OnGetFilterText;

	/** The mode that this skeleton tree is in */
	ESkeletonTreeMode Mode;

	/** Pinned commands panel */
	TSharedPtr<IPinnedCommandList> PinnedCommands;

	/** Context name used to persist settings */
	FName ContextName;

	friend struct FScopedSavedSelection;
}; 
PRAGMA_ENABLE_DEPRECATION_WARNINGS
