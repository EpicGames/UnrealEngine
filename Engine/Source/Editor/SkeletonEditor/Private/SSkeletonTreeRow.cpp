// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSkeletonTreeRow.h"

#include "Framework/SlateDelegates.h"
#include "ISkeletonTree.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Preferences/PersonaOptions.h"
#include "SlotBase.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

class FDragDropEvent;
class STableViewBase;
class SWidget;
struct FGeometry;
struct FPointerEvent;

void SSkeletonTreeRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	Item = InArgs._Item;
	OnDraggingItem = InArgs._OnDraggingItem;
	FilterText = InArgs._FilterText;

	check( Item.IsValid() );

	const FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.Style( FAppStyle::Get(), "TableView.AlternatingRow" );
	
	SMultiColumnTableRow< TSharedPtr<ISkeletonTreeItem> >::Construct( Args, InOwnerTableView );
}

void SSkeletonTreeRow::ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	STableRow<TSharedPtr<ISkeletonTreeItem>>::Content = InContent;

	TSharedRef<SWidget> InlineEditWidget = Item.Pin()->GenerateInlineEditWidget(FilterText, FIsSelected::CreateSP(this, &STableRow::IsSelected));

	// MultiColumnRows let the user decide which column should contain the expander/indenter item.
	this->ChildSlot
		.Padding(InPadding)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InContent
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InlineEditWidget
			]
		];
}

TSharedRef< SWidget > SSkeletonTreeRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == ISkeletonTree::Columns::Name )
	{
		TSharedPtr< SHorizontalBox > RowBox;

		SAssignNew( RowBox, SHorizontalBox )
			.Visibility_Lambda([this]()
			{
				return Item.Pin()->GetFilterResult() == ESkeletonTreeFilterResult::ShownDescendant && GetMutableDefault<UPersonaOptions>()->bHideParentsWhenFiltering ? EVisibility::Collapsed : EVisibility::Visible;
			});

		RowBox->AddSlot()
			.AutoWidth()
			[
				SNew( SExpanderArrow, SharedThis(this) )
				.ShouldDrawWires(true)
			];

		Item.Pin()->GenerateWidgetForNameColumn( RowBox, FilterText, FIsSelected::CreateSP(this, &STableRow::IsSelectedExclusively ) );

		return RowBox.ToSharedRef();
	}
	else
	{
		return Item.Pin()->GenerateWidgetForDataColumn(ColumnName, FIsSelected::CreateSP(this, &STableRow::IsSelectedExclusively ));
	}
}

void SSkeletonTreeRow::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	Item.Pin()->HandleDragEnter(DragDropEvent);
}

void SSkeletonTreeRow::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	Item.Pin()->HandleDragLeave(DragDropEvent);
}

FReply SSkeletonTreeRow::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	return Item.Pin()->HandleDrop(DragDropEvent);
}

FReply SSkeletonTreeRow::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( OnDraggingItem.IsBound() )
	{
		return OnDraggingItem.Execute( MyGeometry, MouseEvent );
	}
	else
	{
		return FReply::Unhandled();
	}
}

int32 SSkeletonTreeRow::DoesItemHaveChildren() const
{
	if(Item.Pin()->HasInlineEditor())
	{
		return 1;
	}

	return SMultiColumnTableRow<TSharedPtr<ISkeletonTreeItem>>::DoesItemHaveChildren();
}

bool SSkeletonTreeRow::IsItemExpanded() const
{
	return SMultiColumnTableRow<TSharedPtr<ISkeletonTreeItem>>::IsItemExpanded() || Item.Pin()->IsInlineEditorExpanded();
}

void SSkeletonTreeRow::ToggleExpansion()
{
	SMultiColumnTableRow<TSharedPtr<ISkeletonTreeItem>>::ToggleExpansion();
	
	if (Item.Pin()->HasInlineEditor())
	{
		Item.Pin()->ToggleInlineEditorExpansion();
		OwnerTablePtr.Pin()->Private_SetItemExpansion(Item.Pin(), Item.Pin()->IsInlineEditorExpanded());
	}
}
