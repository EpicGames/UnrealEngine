// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Delegates/Delegate.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISkeletonTreeItem.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDragDropEvent;
class STableViewBase;
class SWidget;
struct FGeometry;
struct FMargin;
struct FPointerEvent;

DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDraggingTreeItem, const FGeometry&, const FPointerEvent&);

class SSkeletonTreeRow : public SMultiColumnTableRow<TSharedPtr<ISkeletonTreeItem>>
{
public:

	SLATE_BEGIN_ARGS(SSkeletonTreeRow) {}

	/** The item for this row **/
	SLATE_ARGUMENT(TSharedPtr<ISkeletonTreeItem>, Item)

	/** Filter text typed by the user into the parent tree's search widget */
	SLATE_ATTRIBUTE(FText, FilterText);

	/** Delegate for dragging items **/
	SLATE_EVENT(FOnDraggingTreeItem, OnDraggingItem);

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	/** Overridden from STableRow. Allows us to generate inline edit widgets. */
	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;

	/** Override OnDragEnter for drag and drop of sockets onto bones */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Override OnDragLeave for drag and drop of sockets onto bones */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	/** Override OnDrop for drag and drop of sockets and meshes onto bones */
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Override DoesItemHaveChildren to add expander to in-line editors */
	virtual int32 DoesItemHaveChildren() const override;

	/** @return true if the corresponding item is expanded; false otherwise */
	virtual bool IsItemExpanded() const override;

	/** Toggle the expansion of the item associated with this row */
	virtual void ToggleExpansion() override;

protected:

	/** Handler for starting a drag/drop action */
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Get the editable skeleton we are editing */
	TSharedRef<class IEditableSkeleton> GetEditableSkeleton() const { return Item.Pin()->GetEditableSkeleton();  }

	/** Get the skeleton tree we are embedded in */
	TSharedRef<class ISkeletonTree> GetSkeletonTree() const { return Item.Pin()->GetSkeletonTree(); }

private:
	/** The item this row is holding */
	TWeakPtr<ISkeletonTreeItem> Item;

	/** Text the user typed into the search box - used for text highlighting */
	TAttribute<FText> FilterText;

	/** Item that we're dragging */
	FOnDraggingTreeItem OnDraggingItem;

	/** Was the user pressing "Alt" when the drag was started? */
	bool bIsAltDrag;
};
