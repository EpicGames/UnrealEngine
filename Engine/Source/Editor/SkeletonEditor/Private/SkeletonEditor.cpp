// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonEditor.h"

#include "DetailLayoutBuilder.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor/EditorEngine.h"
#include "EngineGlobals.h"
#include "ISkeletonTree.h"
#include "ISkeletonEditorModule.h"
#include "IPersonaToolkit.h"
#include "PersonaModule.h"
#include "SkeletonEditorMode.h"
#include "IAnimationSequenceBrowser.h"
#include "IPersonaPreviewScene.h"
#include "SkeletonEditorCommands.h"
#include "IAssetFamily.h"
#include "PersonaCommonCommands.h"
#include "IEditableSkeleton.h"
#include "ISkeletonTreeItem.h"
#include "Algo/Transform.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "IPersonaViewport.h"
#include "PersonaToolMenuContext.h"
#include "ToolMenus.h"
#include "ToolMenuMisc.h"
#include "SkeletonToolMenuContext.h"
#include "Tools/EdModeInteractiveToolsContext.h"

const FName SkeletonEditorAppIdentifier = FName(TEXT("SkeletonEditorApp"));

const FName SkeletonEditorModes::SkeletonEditorMode(TEXT("SkeletonEditorMode"));

const FName SkeletonEditorTabs::DetailsTab(TEXT("DetailsTab"));
const FName SkeletonEditorTabs::SkeletonTreeTab(TEXT("SkeletonTreeView"));
const FName SkeletonEditorTabs::ViewportTab(TEXT("Viewport"));
const FName SkeletonEditorTabs::AssetBrowserTab(TEXT("SequenceBrowser"));
const FName SkeletonEditorTabs::AnimNotifiesTab(TEXT("SkeletonAnimNotifies"));
const FName SkeletonEditorTabs::CurveMetadataTab(TEXT("AnimCurveMetadataEditorTab"));
const FName SkeletonEditorTabs::CurveDebuggerTab("AnimCurveViewerTab");
const FName SkeletonEditorTabs::AdvancedPreviewTab(TEXT("AdvancedPreviewTab"));
const FName SkeletonEditorTabs::RetargetManagerTab(TEXT("RetargetManager"));
const FName SkeletonEditorTabs::SlotNamesTab("SkeletonSlotNames");
const FName SkeletonEditorTabs::FindReplaceTab("FindReplaceTab");

DEFINE_LOG_CATEGORY(LogSkeletonEditor);

#define LOCTEXT_NAMESPACE "SkeletonEditor"

FSkeletonEditor::FSkeletonEditor()
{
	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}
}

FSkeletonEditor::~FSkeletonEditor()
{
	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->UnregisterForUndo(this);
	}
	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}

void FSkeletonEditor::HandleOpenNewAsset(UObject* InNewAsset)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InNewAsset);
}

void FSkeletonEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SkeletonEditor", "Skeleton Editor"));

	FAssetEditorToolkit::RegisterTabSpawners( InTabManager );
}

void FSkeletonEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FSkeletonEditor::InitSkeletonEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USkeleton* InSkeleton)
{
	Skeleton = InSkeleton;
	
	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneSettingsCustomized = FOnPreviewSceneSettingsCustomized::FDelegate::CreateSP(this, &FSkeletonEditor::HandleOnPreviewSceneSettingsCustomized);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InSkeleton, PersonaToolkitArgs);

	PersonaToolkit->GetPreviewScene()->SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode::ReferencePose);

	PersonaModule.RecordAssetOpened(FAssetData(InSkeleton));

	FSkeletonTreeArgs SkeletonTreeArgs;
	SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(this, &FSkeletonEditor::HandleSelectionChanged);
	SkeletonTreeArgs.PreviewScene = PersonaToolkit->GetPreviewScene();
	SkeletonTreeArgs.ContextName = GetToolkitFName();

	// Extend the skeleton tree's right-click menu with Skeleton-Editor-specific asset-level actions.
	// Other hosts (Skeletal Mesh Editor, AnimBP editor, Animation Editor) create their own SSkeletonTree
	// without this extender, so the entries below are scoped to the Skeleton Editor only.
	SkeletonTreeArgs.Extenders = MakeShared<FExtender>();
	SkeletonTreeArgs.Extenders->AddMenuExtension(
		TEXT("SkeletonTreeContextMenu"),
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateLambda([](FMenuBuilder& InMenuBuilder)
		{
			InMenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("AssetSection", "Asset"));
			{
				InMenuBuilder.AddMenuEntry(FSkeletonEditorCommands::Get().RemoveUnusedBones);
				InMenuBuilder.AddMenuEntry(FSkeletonEditorCommands::Get().UpdateSkeletonRefPose);
			}
			InMenuBuilder.EndSection();
		}));

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(PersonaToolkit->GetSkeleton(), SkeletonTreeArgs);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, SkeletonEditorAppIdentifier, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InSkeleton);

	BindCommands();

	AddApplicationMode(
		SkeletonEditorModes::SkeletonEditorMode,
		MakeShareable(new FSkeletonEditorMode(SharedThis(this), SkeletonTree.ToSharedRef())));

	SetCurrentMode(SkeletonEditorModes::SkeletonEditorMode);

	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	PersonaToolkit->GetPreviewScene()->SetAllowMeshHitProxies(false);
}

FName FSkeletonEditor::GetToolkitFName() const
{
	return FName("SkeletonEditor");
}

FText FSkeletonEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "SkeletonEditor");
}

FString FSkeletonEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "SkeletonEditor ").ToString();
}

FLinearColor FSkeletonEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FSkeletonEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UPersonaToolMenuContext* PersonaContext = NewObject<UPersonaToolMenuContext>();
	PersonaContext->SetToolkit(GetPersonaToolkit());
	MenuContext.AddObject(PersonaContext);

	USkeletonToolMenuContext* SkeletonContext = NewObject<USkeletonToolMenuContext>();
	SkeletonContext->SkeletonEditor = SharedThis(this);
	MenuContext.AddObject(SkeletonContext);
}

void FSkeletonEditor::BindCommands()
{
	FSkeletonEditorCommands::Register();
	
	ToolkitCommands->MapAction( FSkeletonEditorCommands::Get().RemoveUnusedBones,
		FExecuteAction::CreateSP(this, &FSkeletonEditor::RemoveUnusedBones),
		FCanExecuteAction::CreateSP(this, &FSkeletonEditor::CanRemoveBones));

	ToolkitCommands->MapAction(FSkeletonEditorCommands::Get().TestSkeletonCurveMetaDataForUse,
		FExecuteAction::CreateSP(this, &FSkeletonEditor::TestSkeletonCurveMetaDataForUse));

	ToolkitCommands->MapAction(FSkeletonEditorCommands::Get().UpdateSkeletonRefPose,
		FExecuteAction::CreateSP(this, &FSkeletonEditor::UpdateSkeletonRefPose));

	ToolkitCommands->MapAction(FSkeletonEditorCommands::Get().AnimNotifyWindow,
		FExecuteAction::CreateSP(this, &FSkeletonEditor::OnAnimNotifyWindow));

	ToolkitCommands->MapAction(FSkeletonEditorCommands::Get().Retargeting,
		FExecuteAction::CreateSP(this, &FSkeletonEditor::OnRetargetManager));

	ToolkitCommands->MapAction(FSkeletonEditorCommands::Get().ImportMesh,
		FExecuteAction::CreateSP(this, &FSkeletonEditor::OnImportAsset));

	ToolkitCommands->MapAction(FPersonaCommonCommands::Get().TogglePlay,
		FExecuteAction::CreateRaw(&GetPersonaToolkit()->GetPreviewScene().Get(), &IPersonaPreviewScene::TogglePlayback));
}

TSharedPtr<FSkeletonEditor> FSkeletonEditor::GetSkeletonEditor(const FToolMenuContext& InMenuContext)
{
	if (USkeletonToolMenuContext* Context = InMenuContext.FindContext<USkeletonToolMenuContext>())
	{
		if (Context->SkeletonEditor.IsValid())
		{
			return StaticCastSharedPtr<FSkeletonEditor>(Context->SkeletonEditor.Pin());
		}
	}

	return TSharedPtr<FSkeletonEditor>();
}

void FSkeletonEditor::ExtendToolbar()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	// Add in Editor Specific functionality
	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	const FToolMenuInsert SectionInsertLocation("Asset", EToolMenuInsertType::After);

	{
		ToolMenu->AddDynamicSection("Persona", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
		{
			TSharedPtr<FSkeletonEditor> SkeletonEditor = GetSkeletonEditor(InToolMenu->Context);
			if (SkeletonEditor.IsValid() && SkeletonEditor->PersonaToolkit.IsValid())
			{
				FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
				PersonaModule.AddCommonToolbarExtensions(InToolMenu);
			}
		}), SectionInsertLocation);
	}

	{
		FToolMenuSection& SkeletonSection = ToolMenu->AddSection("Skeleton", LOCTEXT("ToolbarSkeletonSectionLabel", "Skeleton"), SectionInsertLocation);
		SkeletonSection.AddEntry(FToolMenuEntry::InitToolBarButton(FSkeletonEditorCommands::Get().AnimNotifyWindow));
		SkeletonSection.AddEntry(FToolMenuEntry::InitToolBarButton(FSkeletonEditorCommands::Get().Retargeting, LOCTEXT("Toolbar_RetargetManager", "Retarget Manager")));
		SkeletonSection.AddEntry(FToolMenuEntry::InitToolBarButton(FSkeletonEditorCommands::Get().ImportMesh));
	}

	// If the ToolbarExtender is valid, remove it before rebuilding it
	if (ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	AddToolbarExtender(SkeletonEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	TArray<ISkeletonEditorModule::FSkeletonEditorToolbarExtender> ToolbarExtenderDelegates = SkeletonEditorModule.GetAllSkeletonEditorToolbarExtenders();

	for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
	{
		if (ToolbarExtenderDelegate.IsBound())
		{
			AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), SharedThis(this)));
		}
	}

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
		{
			FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
			TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(Skeleton);
			AddToolbarWidget(PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily));
		}	
	));
}

void FSkeletonEditor::ExtendMenu()
{
	MenuExtender = MakeShareable(new FExtender);

	struct Local
	{
		static void ExtendMenu(FMenuBuilder& MenuBuilder)
		{
			// View
			MenuBuilder.BeginSection("SkeletonEditor", LOCTEXT("SkeletonEditorAssetMenu_Skeleton", "Skeleton"));
			{
				MenuBuilder.AddMenuEntry(FSkeletonEditorCommands::Get().RemoveUnusedBones);
				MenuBuilder.AddMenuEntry(FSkeletonEditorCommands::Get().UpdateSkeletonRefPose);
				MenuBuilder.AddMenuEntry(FSkeletonEditorCommands::Get().TestSkeletonCurveMetaDataForUse);
			}
			MenuBuilder.EndSection();
		}
	};

	MenuExtender->AddMenuExtension(
		"AssetEditorActions",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateStatic(&Local::ExtendMenu)
		);

	AddMenuExtender(MenuExtender);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	AddMenuExtender(SkeletonEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

UObject* FSkeletonEditor::HandleGetAsset()
{
	return GetEditingObject();
}

void FSkeletonEditor::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObjects(InObjects);
	}
}

void FSkeletonEditor::HandleObjectSelected(UObject* InObject)
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(InObject);
	}
}

void FSkeletonEditor::HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	RouteTreeSelectionToModeManager(InSelectedItems);

	if (DetailsView.IsValid())
	{
		TArray<UObject*> Objects;
		Algo::TransformIf(InSelectedItems, Objects, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() != nullptr; }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject(); });
		DetailsView->SetObjects(Objects);
	}
}

void FSkeletonEditor::PostUndo(bool bSuccess)
{
	OnPostUndo.Broadcast();
}

void FSkeletonEditor::PostRedo(bool bSuccess)
{
	OnPostUndo.Broadcast();
}

void FSkeletonEditor::Tick(float DeltaTime)
{
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

TStatId FSkeletonEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FSkeletonEditor, STATGROUP_Tickables);
}

bool FSkeletonEditor::CanRemoveBones() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
	return PreviewMeshComponent && PreviewMeshComponent->GetSkeletalMeshAsset();
}

void FSkeletonEditor::RemoveUnusedBones()
{
	GetSkeletonTree()->GetEditableSkeleton()->RemoveUnusedBones();
}

void FSkeletonEditor::TestSkeletonCurveMetaDataForUse() const
{
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaModule.TestSkeletonCurveMetaDataForUse(GetSkeletonTree()->GetEditableSkeleton());
}

void FSkeletonEditor::UpdateSkeletonRefPose()
{
	if (PersonaToolkit->GetPreviewMeshComponent()->GetSkeletalMeshAsset())
	{
		GetSkeletonTree()->GetEditableSkeleton()->UpdateSkeletonReferencePose(PersonaToolkit->GetPreviewMeshComponent()->GetSkeletalMeshAsset());
	}
}

void FSkeletonEditor::OnAnimNotifyWindow()
{
	InvokeTab(SkeletonEditorTabs::AnimNotifiesTab);
}

void FSkeletonEditor::OnRetargetManager()
{
	InvokeTab(SkeletonEditorTabs::RetargetManagerTab);
}

void FSkeletonEditor::OnImportAsset()
{
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaModule.ImportNewAsset(Skeleton, FBXIT_SkeletalMesh);
}

void FSkeletonEditor::HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideCategory("Animation Blueprint");
}

void FSkeletonEditor::HandleDetailsCreated(const TSharedRef<IDetailsView>& InDetailsView)
{
	DetailsView = InDetailsView;
}

void FSkeletonEditor::HandleAnimationSequenceBrowserCreated(const TSharedRef<IAnimationSequenceBrowser>& InSequenceBrowser)
{
	SequenceBrowser = InSequenceBrowser;
}

void FSkeletonEditor::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InViewport)
{
	Viewport = InViewport;

	// we need the viewport client to start out focused, or else it won't get ticked until we click inside it.
	InViewport->GetViewportClient().ReceivedFocus(InViewport->GetViewportClient().Viewport);
}

void FSkeletonEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	if (Viewport.IsValid())
	{
		Viewport->AddOverlayWidget(InOverlaidWidget, ZOrder);
	}
}

void FSkeletonEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget)
{
	if (Viewport.IsValid())
	{
		Viewport->RemoveOverlayWidget(InOverlaidWidget);
	}
}

IAnimationSequenceBrowser* FSkeletonEditor::GetAssetBrowser() const
{
	return SequenceBrowser.Pin().Get();
}

#undef LOCTEXT_NAMESPACE
