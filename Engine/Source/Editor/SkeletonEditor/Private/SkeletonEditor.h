// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Toolkits/IToolkitHost.h"
#include "TickableEditorObject.h"
#include "EditorUndoClient.h"
#include "IDetailsView.h"
#include "ISkeletonEditor.h"
#include "Containers/ArrayView.h"
#include "IAnimationSequenceBrowser.h"

class IDetailLayoutBuilder;
class IPersonaToolkit;
class IPersonaViewport;
class ISkeletonTree;
class USkeleton;
class ISkeletonTreeItem;

struct FToolMenuContext;

namespace SkeletonEditorModes
{
	// Mode identifiers
	extern const FName SkeletonEditorMode;
}

namespace SkeletonEditorTabs
{
	// Tab identifiers
	extern const FName DetailsTab;
	extern const FName SkeletonTreeTab;
	extern const FName ViewportTab;
	extern const FName AssetBrowserTab;
	extern const FName AnimNotifiesTab;
	extern const FName CurveMetadataTab;
	extern const FName CurveDebuggerTab;
	extern const FName AdvancedPreviewTab;
	extern const FName RetargetManagerTab;
	extern const FName SlotNamesTab;
	extern const FName FindReplaceTab;
}

class FSkeletonEditor : public ISkeletonEditor, public FEditorUndoClient, public FTickableEditorObject
{
public:
	FSkeletonEditor();

	virtual ~FSkeletonEditor();

	/** Edits the specified Skeleton object */
	void InitSkeletonEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, class USkeleton* InSkeleton);

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }

	/** IToolkit interface */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	virtual IAnimationSequenceBrowser* GetAssetBrowser() const override;

	/** Handle opening a new asset from the asset browser */
	void HandleOpenNewAsset(UObject* InNewAsset);

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** FTickableEditorObject Interface */
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("AnimatingObjects/SkeletalMeshAnimation/Persona/Modes/Skeleton"));
	}

	/** Get the skeleton tree widget */
	TSharedRef<class ISkeletonTree> GetSkeletonTree() const { return SkeletonTree.ToSharedRef(); }

	UObject* HandleGetAsset();

public:
	void HandleObjectsSelected(const TArray<UObject*>& InObjects);

	void HandleObjectSelected(UObject* InObject);

	void HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);

	void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);

	void HandleAnimationSequenceBrowserCreated(const TSharedRef<class IAnimationSequenceBrowser>& InSequenceBrowser);

	void HandleViewportCreated(const TSharedRef<IPersonaViewport>& InViewport);

	// FAssetEditorToolkit interface
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE) override;
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget) override;

private:
	void ExtendMenu();

	void ExtendToolbar();

	void BindCommands();

	bool CanRemoveBones() const;

	void RemoveUnusedBones();

	void TestSkeletonCurveMetaDataForUse() const;

	void UpdateSkeletonRefPose();

	void OnAnimNotifyWindow();

	void OnRetargetManager();

	void OnImportAsset();

	void HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder);

	static TSharedPtr<FSkeletonEditor> GetSkeletonEditor(const FToolMenuContext& InMenuContext);
public:
	/** Multicast delegate fired on global undo/redo */
	FSimpleMulticastDelegate OnPostUndo;

	/** Multicast delegate fired on curves changing */
	FSimpleMulticastDelegate OnCurvesChanged;

private:
	/** The skeleton we are editing */
	USkeleton* Skeleton;

	/** Toolbar extender */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Menu extender */
	TSharedPtr<FExtender> MenuExtender;

	/** Persona toolkit */
	TSharedPtr<class IPersonaToolkit> PersonaToolkit;

	/** Skeleton tree */
	TSharedPtr<class ISkeletonTree> SkeletonTree;

	/** Viewport */
	TSharedPtr<class IPersonaViewport> Viewport;

	/** Details panel */
	TSharedPtr<class IDetailsView> DetailsView;

	/** Sequence Browser **/
	TWeakPtr<class IAnimationSequenceBrowser> SequenceBrowser;
};
