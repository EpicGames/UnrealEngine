// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletonEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "SkeletonEditorCommands"

void FSkeletonEditorCommands::RegisterCommands()
{
	// skeleton menu
	UI_COMMAND(ChangeSkeletonPreviewMesh, "Set Preview Mesh as Default", "Changes the skeletons default preview mesh to the current open preview mesh. The skeleton will require saving after this action.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveUnusedBones, "Remove Unused Bones from Skeleton", "Removes any bones from the skeleton that are not used by any of its meshes. The skeleton and associated animations will require saving after this action.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TestSkeletonCurveMetaDataForUse, "Test Skeleton Curve Metadata for Use", "Iterates over all Animations and Meshes for this skeleton to look for unused morph/material curves", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AnimNotifyWindow, "Anim Notifies", "You can manage animation notifies that belong to the skeleton.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Retargeting, "Retargeting", "Edit Compatible Skeletons and Retarget Sources for skeletal meshes using this skeleton. ", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportMesh, "Import Mesh", "Import new mesh for this skeleton. ", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(UpdateSkeletonRefPose, "Update Skeleton RefPose", "Update Skeleton ref pose based on current preview mesh", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
