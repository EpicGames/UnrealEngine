// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FSkeletonEditorCommands : public TCommands<FSkeletonEditorCommands>
{
public:
	FSkeletonEditorCommands()
		: TCommands<FSkeletonEditorCommands>(TEXT("SkeletonEditor"), NSLOCTEXT("Contexts", "SkeletonEditor", "Skeleton Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

public:

	// skeleton menu options
	// Command to allow users to set the skeletons preview mesh
	TSharedPtr<FUICommandInfo> ChangeSkeletonPreviewMesh;
	// Command to allow users to remove unused bones (not referenced by any skeletalmesh) from the skeleton
	TSharedPtr<FUICommandInfo> RemoveUnusedBones;
	// Command to allow users to look for unused curves (curves that exist in animations that do not drive any morph/materials
	TSharedPtr<FUICommandInfo> TestSkeletonCurveMetaDataForUse;
	// Command to show Anim Notify window
	TSharedPtr<FUICommandInfo> AnimNotifyWindow;
	// Command to show retargeting tools
	TSharedPtr<FUICommandInfo> Retargeting;
	// Import Mesh for this Skeleton
	TSharedPtr<FUICommandInfo> ImportMesh;

	// Command to allow users to remove unused bones (not referenced by any skeletalmesh) from the skeleton
	TSharedPtr<FUICommandInfo> UpdateSkeletonRefPose;
};
