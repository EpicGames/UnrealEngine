// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonEditorMode.h"

#include "AnimAssetFindReplace.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "SkeletonEditor.h"
#include "ISkeletonTree.h"
#include "ISkeletonEditorModule.h"
#include "IPersonaToolkit.h"
#include "AnimAssetFindReplace.h"

FSkeletonEditorMode::FSkeletonEditorMode(TSharedRef<FWorkflowCentricApplication> InHostingApp, TSharedRef<ISkeletonTree> InSkeletonTree)
	: FApplicationMode(SkeletonEditorModes::SkeletonEditorMode)
{
	FWorkflowAllowedTabSet& TabFactories = ApplicationModeTabFactories;

	TSharedRef<FSkeletonEditor> SkeletonEditor = StaticCastSharedRef<FSkeletonEditor>(InHostingApp);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TabFactories.RegisterFactory(SkeletonEditorModule.CreateSkeletonTreeTabFactory(InHostingApp, InSkeletonTree));

	FOnObjectsSelected OnObjectsSelected = FOnObjectsSelected::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleObjectsSelected);
	FOnObjectSelected OnObjectSelected = FOnObjectSelected::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleObjectSelected);
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleDetailsCreated)));

	FPersonaViewportArgs ViewportArgs(SkeletonEditor->GetPersonaToolkit()->GetPreviewScene());
	ViewportArgs.ContextName = TEXT("SkeletonEditor.Viewport");
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleViewportCreated);

	PersonaModule.RegisterPersonaViewportTabFactories(TabFactories, InHostingApp, ViewportArgs);

	TabFactories.RegisterFactory(PersonaModule.CreateAnimNotifiesTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(),  OnObjectsSelected));
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, SkeletonEditor->GetPersonaToolkit()->GetPreviewScene()));
	TabFactories.RegisterFactory(PersonaModule.CreateRetargetSourcesTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(), SkeletonEditor->OnPostUndo));
	TabFactories.RegisterFactory(PersonaModule.CreateCurveMetadataEditorTabFactory(InHostingApp, SkeletonEditor->HandleGetAsset(), SkeletonEditor->GetPersonaToolkit()->GetPreviewScene(), OnObjectsSelected));
	TabFactories.RegisterFactory(PersonaModule.CreateSkeletonSlotNamesTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(), OnObjectSelected));
	TabFactories.RegisterFactory(PersonaModule.CreateAssetDetailsTabFactory(InHostingApp, FOnGetAsset::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleGetAsset), FOnDetailsCreated()));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimationAssetBrowserTabFactory(InHostingApp, SkeletonEditor->GetPersonaToolkit(), FOnOpenNewAsset::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleOpenNewAsset), FOnAnimationSequenceBrowserCreated::CreateSP(&SkeletonEditor.Get(), &FSkeletonEditor::HandleAnimationSequenceBrowserCreated), true));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimAssetFindReplaceTabFactory(InHostingApp, FAnimAssetFindReplaceConfig()));
	TabFactories.RegisterFactory(PersonaModule.CreateCurveViewerTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(), SkeletonEditor->GetPersonaToolkit()->GetPreviewScene(), OnObjectsSelected));

	TabLayout = FTabManager::NewLayout("Standalone_SkeletonEditor_Layout_v1.5")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->SetHideTabWell(true)
					->AddTab(SkeletonEditorTabs::SkeletonTreeTab, ETabState::OpenedTab)
					->AddTab(SkeletonEditorTabs::RetargetManagerTab, ETabState::ClosedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.6f)
					->SetOrientation(Orient_Vertical)
					->Split
					(	
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(true)
						->AddTab(SkeletonEditorTabs::ViewportTab, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(SkeletonEditorTabs::FindReplaceTab, ETabState::ClosedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(SkeletonEditorTabs::DetailsTab, ETabState::OpenedTab)
						->AddTab(SkeletonEditorTabs::CurveDebuggerTab, ETabState::ClosedTab)
						->AddTab(SkeletonEditorTabs::AdvancedPreviewTab, ETabState::OpenedTab)
						->SetForegroundTab(SkeletonEditorTabs::DetailsTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(false)
						->AddTab(SkeletonEditorTabs::AnimNotifiesTab, ETabState::OpenedTab)
						->AddTab(SkeletonEditorTabs::AssetBrowserTab, ETabState::OpenedTab)
						->AddTab(SkeletonEditorTabs::CurveMetadataTab, ETabState::OpenedTab)
						->AddTab(SkeletonEditorTabs::SlotNamesTab, ETabState::ClosedTab)
					)
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FSkeletonEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	if (TSharedPtr<FWorkflowCentricApplication> HostingApp = GetHost())
	{
		TSharedRef<FTabManager> TabManager = InTabManager.ToSharedRef();
		HostingApp->RegisterTabSpawners(TabManager);
		RegisterTabFactoriesWithAppAndManager(HostingApp.Get(), TabManager);
	}
}
