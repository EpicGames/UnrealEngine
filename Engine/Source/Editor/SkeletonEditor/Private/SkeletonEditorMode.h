// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FSkeletonEditorMode : public FApplicationMode
{
public:
	FSkeletonEditorMode(TSharedRef<class FWorkflowCentricApplication> InHostingApp, TSharedRef<class ISkeletonTree> InSkeletonTree);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
};
