// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWidget.h"
#include "Application/SlateWindowHelper.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ISkeletonEditor.h"
#include "ISkeletonTree.h"
#include "ISkeletonEditorModule.h"
#include "SkeletonEditor.h"
#include "IEditableSkeleton.h"
#include "EditableSkeleton.h"
#include "SkeletonTreeManager.h"
#include "SkeletonTreeSummoner.h"
#include "BoneProxy.h"
#include "BoneProxyDetailsCustomization.h"
#include "PropertyEditorModule.h"

class FSkeletonEditorModule : public ISkeletonEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FSkeletonEditorModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UBoneProxy::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FBoneProxyDetailsCustomization::MakeInstance));
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			if (UObjectInitialized())
			{
				PropertyModule.UnregisterCustomClassLayout(UBoneProxy::StaticClass()->GetFName());
			}
		}
	}

	virtual TSharedRef<ISkeletonEditor> CreateSkeletonEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USkeleton* InSkeleton) override
	{
		TSharedRef<FSkeletonEditor> SkeletonEditor(new FSkeletonEditor());
		SkeletonEditor->InitSkeletonEditor(Mode, InitToolkitHost, InSkeleton);
		return SkeletonEditor;
	}

	virtual TSharedRef<ISkeletonTree> CreateSkeletonTree(USkeleton* InSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs) override
	{
		return FSkeletonTreeManager::Get().CreateSkeletonTree(InSkeleton, InSkeletonTreeArgs);
	}

	virtual TSharedRef<ISkeletonTree> CreateSkeletonTree(const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs) override
	{
		return FSkeletonTreeManager::Get().CreateSkeletonTree(InEditableSkeleton, InSkeletonTreeArgs);
	}

	virtual TSharedRef<class FWorkflowTabFactory> CreateSkeletonTreeTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<ISkeletonTree>& InSkeletonTree) override
	{
		return MakeShareable(new FSkeletonTreeSummoner(InHostingApp, InSkeletonTree));
	}

	virtual TSharedRef<IEditableSkeleton> CreateEditableSkeleton(USkeleton* InSkeleton) override
	{
		return FSkeletonTreeManager::Get().CreateEditableSkeleton(InSkeleton);
	}

	virtual TSharedRef<SWidget> CreateBlendProfilePicker(USkeleton* InSkeleton, const FBlendProfilePickerArgs& InArgs) override
	{
		TSharedRef<FEditableSkeleton> EditableSkeleton = FSkeletonTreeManager::Get().CreateEditableSkeleton(InSkeleton);
		return EditableSkeleton->CreateBlendProfilePicker(InArgs);
	}

	virtual TArray<FSkeletonEditorToolbarExtender>& GetAllSkeletonEditorToolbarExtenders() override { return SkeletonEditorToolbarExtenders; }

	/** Gets the extensibility managers for outside entities to extend this editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:
	/** Extensibility managers */
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	TArray<FSkeletonEditorToolbarExtender> SkeletonEditorToolbarExtenders;
};

IMPLEMENT_MODULE(FSkeletonEditorModule, SkeletonEditor);
