// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeAttachedAssetItem.h"
#include "PersonaUtils.h"
#include "IPersonaPreviewScene.h"
#include "AssetSelection.h"
#include "Styling/SlateIconFinder.h"
#include "Modules/ModuleManager.h"
#include "ContentBrowserModule.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "ActorFactories/ActorFactory.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "IContentBrowserSingleton.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreeAttachedAssetItem"

FSkeletonTreeAttachedAssetItem::FSkeletonTreeAttachedAssetItem(UObject* InAsset, const FName& InAttachedTo, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreeItem(InSkeletonTree)
	, AttachedTo(InAttachedTo)
	, Asset(InAsset)
{
	AssetComponent = PersonaUtils::GetComponentForAttachedObject(InSkeletonTree->GetPreviewScene()->GetPreviewMeshComponent(), InAsset, InAttachedTo);
}

void FSkeletonTreeAttachedAssetItem::GenerateWidgetForNameColumn( TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected )
{
	UActorFactory* ActorFactory = FActorFactoryAssetProxy::GetFactoryForAssetObject( Asset );
	const FSlateBrush* IconBrush = ActorFactory ? FSlateIconFinder::FindIconBrushForClass(ActorFactory->GetDefaultActorClass(FAssetData())) : nullptr;
	
	Box->AddSlot()
		.Padding(FMargin(0.0f, 2.0f))
		.AutoWidth()
		[
			SNew( SImage )
			.Image( IconBrush )
		];

	const FSlateFontInfo TextFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	const FLinearColor TextColor = FLinearColor::White;

	Box->AddSlot()
		.AutoWidth()
		[
			SNew( STextBlock )
			.ColorAndOpacity(TextColor)
			.Text( FText::FromString(Asset->GetName()) )
			.HighlightText( FilterText )
			.Font(TextFont)
		];

	Box->AddSlot()
		.AutoWidth()
		.Padding(5.0f,0.0f)
		[
			SNew( STextBlock )
			.ColorAndOpacity( FLinearColor::Gray )
			.Text(LOCTEXT( "AttachedAssetPreviewText", "[Preview Only]" ) )
			.Font(TextFont)
			.ToolTipText( LOCTEXT( "AttachedAssetPreviewText_ToolTip", "Attached assets in Persona are preview only and do not carry through to the game." ) )
		];
}

TSharedRef< SWidget > FSkeletonTreeAttachedAssetItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	if(DataColumnName == ISkeletonTree::Columns::Retargeting)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("TranslationCheckBoxToolTip", "Click to toggle visibility of this asset"))
				.OnCheckStateChanged(this, &FSkeletonTreeAttachedAssetItem::OnToggleAssetDisplayed)
				.IsChecked(this, &FSkeletonTreeAttachedAssetItem::IsAssetDisplayed)
				.Style(FAppStyle::Get(), "CheckboxLookToggleButtonCheckbox")
				[
					SNew(SImage)
					.Image(this, &FSkeletonTreeAttachedAssetItem::OnGetAssetDisplayedButtonImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return SNullWidget::NullWidget;
}

ECheckBoxState FSkeletonTreeAttachedAssetItem::IsAssetDisplayed() const
{
	if(AssetComponent.IsValid())
	{
		return AssetComponent->IsVisible() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FSkeletonTreeAttachedAssetItem::OnToggleAssetDisplayed( ECheckBoxState InCheckboxState )
{
	if(AssetComponent.IsValid())
	{
		AssetComponent->SetVisibility(InCheckboxState == ECheckBoxState::Checked);
	}
}

const FSlateBrush* FSkeletonTreeAttachedAssetItem::OnGetAssetDisplayedButtonImage() const
{
	return IsAssetDisplayed() == ECheckBoxState::Checked ?
		FAppStyle::GetBrush( "Kismet.VariableList.ExposeForInstance" ) :
		FAppStyle::GetBrush( "Kismet.VariableList.HideForInstance" );
}

void FSkeletonTreeAttachedAssetItem::OnItemDoubleClicked()
{
	TArray<UObject*> AssetsToSync;
	AssetsToSync.Add(Asset);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets( AssetsToSync );
}

#undef LOCTEXT_NAMESPACE
