// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ISkeletonTreeItem.h"
#include "SkeletonTreeItem.h"
#include "AssetRegistry/AssetData.h"

class USceneComponent;

class FSkeletonTreeAttachedAssetItem : public FSkeletonTreeItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreeAttachedAssetItem, FSkeletonTreeItem)

	FSkeletonTreeAttachedAssetItem(UObject* InAsset, const FName& InAttachedTo, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	/** ISkeletonTreeItem interface */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override;
	virtual FName GetRowItemName() const { return Asset->GetFName(); }
	virtual FName GetAttachName() const { return GetParentName(); }
	virtual void OnItemDoubleClicked() override;

	/** Returns the name of the socket/bone this asset is attached to */
	const FName& GetParentName() const { return AttachedTo; }

	/** Return the asset this info represents */
	UObject* GetAsset() const { return Asset; }

private:
	/** Accessor for SCheckBox **/
	ECheckBoxState IsAssetDisplayed() const;

	/** Called when user toggles checkbox **/
	void OnToggleAssetDisplayed(ECheckBoxState InCheckboxState);

	/** Called when we need to get the state-based-image to show for the asset displayed checkbox */
	const FSlateBrush* OnGetAssetDisplayedButtonImage() const;

private:
	/** The name of the socket/bone this asset is attached to */
	const FName AttachedTo;

	/** The attached asset that this tree item represents */
	UObject* Asset;

	/** The component of the attached asset */
	TWeakObjectPtr<USceneComponent> AssetComponent;
};
