// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeBoneItem.h"
#include "SSkeletonTreeRow.h"
#include "IPersonaPreviewScene.h"
#include "IDocumentation.h"
#include "Animation/BlendProfile.h"
#include "IEditableSkeleton.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "BoneDragDropOp.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "SocketDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/Package.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Preferences/PersonaOptions.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreeBoneItem"

FSkeletonTreeBoneItem::FSkeletonTreeBoneItem(const FName& InBoneName, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreeItem(InSkeletonTree)
	, BoneName(InBoneName)
	, bWeightedBone(false)
	, bRequiredBone(false)
	, bBlendSliderStartedTransaction(false)
{
	static const FString BoneProxyPrefix(TEXT("BONEPROXY_"));

	BoneProxy = NewObject<UBoneProxy>(GetTransientPackage(), *(BoneProxyPrefix + FString::Printf(TEXT("%p"), &InSkeletonTree.Get()) + InBoneName.ToString()));
	BoneProxy->SetFlags(RF_Transactional);
	BoneProxy->BoneName = InBoneName;
	const TSharedPtr<IPersonaPreviewScene> PreviewScene = InSkeletonTree->GetPreviewScene();
	if (PreviewScene.IsValid())
	{
		BoneProxy->SkelMeshComponent = PreviewScene->GetPreviewMeshComponent();
		BoneProxy->WeakPreviewScene = PreviewScene.ToWeakPtr();
	}
}

const FSlateBrush* FSkeletonTreeBoneItem::GetLODIcon() const
{
	if (!bRequiredBone)
	{
		return FAppStyle::Get().GetBrush("SkeletonTree.NonRequiredBone");
	}

	else if (!bWeightedBone)
	{
		return FAppStyle::Get().GetBrush("SkeletonTree.BoneNonWeighted");
	}

	return FAppStyle::Get().GetBrush("SkeletonTree.Bone");

}

void FSkeletonTreeBoneItem::GenerateWidgetForNameColumn( TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected )
{
	const FSlateBrush* LODIcon = FAppStyle::GetBrush("SkeletonTree.Bone");

	Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 2.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ColorAndOpacity(this, &FSkeletonTreeBoneItem::GetBoneTextColor, InIsSelected)
			.Image(this, &FSkeletonTreeBoneItem::GetLODIcon)
		];

	if (GetSkeletonTree()->GetPreviewScene().IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent();
		CacheLODChange(PreviewComponent);
	}	

	FText ToolTip = GetBoneToolTip();
	Box->AddSlot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew( STextBlock )
			.ColorAndOpacity(this, &FSkeletonTreeBoneItem::GetBoneTextColor, InIsSelected)
			.Text( FText::FromName(BoneName) )
			.HighlightText( FilterText )
			.Font(this, &FSkeletonTreeBoneItem::GetBoneTextFont)
			.ToolTipText( ToolTip )
		];

	Box->AddSlot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew (STextBlock)
			.ColorAndOpacity(this, &FSkeletonTreeBoneItem::GetBoneTextColor, InIsSelected)
			.Text_Lambda( [BoneName=BoneName, EditableSkeleton=GetEditableSkeleton()]() -> FText
			{
				const int32 BoneIndex = EditableSkeleton->GetSkeleton().GetReferenceSkeleton().FindBoneIndex(BoneName);
				return FText::Format(LOCTEXT("BoneIndexValue", "<{0}>"), FText::AsNumber(BoneIndex));
			})
			.Font(this, &FSkeletonTreeBoneItem::GetBoneTextFont)
			.ToolTipText( LOCTEXT("BoneIndexTooltip", "The index of this bone in the reference skeleton's flat list of bones.") )
			.Visibility_Lambda([]() -> EVisibility
			{
				return GetDefault<UPersonaOptions>()->bShowBoneIndexes ? EVisibility::Visible : EVisibility::Collapsed;
			})
		];
}

TSharedRef< SWidget > FSkeletonTreeBoneItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	if(DataColumnName == ISkeletonTree::Columns::Retargeting)
	{
		return 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(0.0f)
			[
				SAssignNew(RetargetingComboButton, SComboButton)
            	.ComboButtonStyle( &FAppStyle::Get().GetWidgetStyle< FComboButtonStyle >("SkeletonTree.RetargetingComboButton"))
				.ForegroundColor(this, &FSkeletonTreeBoneItem::GetBoneTextColor, InIsSelected)
				.ContentPadding(0.f)
				.OnGetMenuContent(this, &FSkeletonTreeBoneItem::CreateBoneTranslationRetargetingModeMenu)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
					LOCTEXT("RetargetingToolTip", "Set bone translation retargeting mode"),
					nullptr,
					TEXT("Shared/Editors/Persona"),
					TEXT("TranslationRetargeting")))
				.VAlign(VAlign_Center)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FSkeletonTreeBoneItem::GetTranslationRetargetingModeMenuTitle)
				]
			];
	}
	else if(DataColumnName == ISkeletonTree::Columns::BlendProfile)
	{
		return SNew(SBox)
			.Padding(0.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SSpinBox<float>)
				.Visibility(this, &FSkeletonTreeBoneItem::GetBoneBlendProfileVisibility)
				.Style(&FAppStyle::Get(), "SkeletonTree.HyperlinkSpinBox")
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
				.ContentPadding(0.0f)
				.Delta(0.01f)
				.MinValue(0.0f)
				.MinSliderValue(this, &FSkeletonTreeBoneItem::GetBlendProfileMinSliderValue)
				.MaxSliderValue(this, &FSkeletonTreeBoneItem::GetBlendProfileMaxSliderValue)
				.Value(this, &FSkeletonTreeBoneItem::GetBoneBlendProfileScale)
				.OnValueCommitted(this, &FSkeletonTreeBoneItem::OnBlendSliderCommitted)
				.OnValueChanged(this, &FSkeletonTreeBoneItem::OnBlendSliderChanged)
				.OnBeginSliderMovement(this, &FSkeletonTreeBoneItem::OnBeginBlendSliderMovement)
				.OnEndSliderMovement(this, &FSkeletonTreeBoneItem::OnEndBlendSliderMovement)
				.ClearKeyboardFocusOnCommit(true)
			];
	}

	return SNullWidget::NullWidget;
}

EVisibility FSkeletonTreeBoneItem::GetBoneBlendProfileVisibility() const
{
	return GetSkeletonTree()->GetSelectedBlendProfile() ? EVisibility::Visible : EVisibility::Collapsed;
}

float FSkeletonTreeBoneItem::GetBoneBlendProfileScale()	const 
{
	if (UBlendProfile* CurrentProfile = GetSkeletonTree()->GetSelectedBlendProfile())
	{
		return CurrentProfile->GetBoneBlendScale(BoneName);
	}

	return 0.0;
}

TOptional<float> FSkeletonTreeBoneItem::GetBlendProfileMaxSliderValue() const
{
	if (UBlendProfile* CurrentProfile = GetSkeletonTree()->GetSelectedBlendProfile())
	{
		return (CurrentProfile->GetMode() == EBlendProfileMode::WeightFactor) ? 10.0f : 1.0f;
	}

	return 1.0f;
}

TOptional<float> FSkeletonTreeBoneItem::GetBlendProfileMinSliderValue() const
{
	if (UBlendProfile* CurrentProfile = GetSkeletonTree()->GetSelectedBlendProfile())
	{
		return (CurrentProfile->GetMode() == EBlendProfileMode::WeightFactor) ? 1.0f : 0.0f;
	}

	return 0.0f;
}

FSlateColor FSkeletonTreeBoneItem::GetRetargetingComboButtonForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");
	static const FName DefaultForegroundName("DefaultForeground");

	if (RetargetingComboButton.IsValid())
	{
		return RetargetingComboButton->IsHovered() ? FAppStyle::GetSlateColor(InvertedForegroundName) : FAppStyle::GetSlateColor(DefaultForegroundName);
	}
	return FSlateColor::UseForeground();
}

TSharedRef< SWidget > FSkeletonTreeBoneItem::CreateBoneTranslationRetargetingModeMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("BoneTranslationRetargetingMode", LOCTEXT( "BoneTranslationRetargetingModeMenuHeading", "Bone Translation Retargeting Mode" ) );
	{
		UEnum* const Enum = StaticEnum<EBoneTranslationRetargetingMode::Type>();	
		check(Enum);

		FUIAction ActionRetargetingAnimation = FUIAction(FExecuteAction::CreateSP(this, &FSkeletonTreeBoneItem::SetBoneTranslationRetargetingMode, EBoneTranslationRetargetingMode::Animation));
		MenuBuilder.AddMenuEntry( Enum->GetDisplayNameTextByValue(EBoneTranslationRetargetingMode::Animation), LOCTEXT( "BoneTranslationRetargetingAnimationToolTip", "Use translation from animation." ), FSlateIcon(), ActionRetargetingAnimation);

		FUIAction ActionRetargetingSkeleton = FUIAction(FExecuteAction::CreateSP(this, &FSkeletonTreeBoneItem::SetBoneTranslationRetargetingMode, EBoneTranslationRetargetingMode::Skeleton));
		MenuBuilder.AddMenuEntry( Enum->GetDisplayNameTextByValue(EBoneTranslationRetargetingMode::Skeleton), LOCTEXT( "BoneTranslationRetargetingSkeletonToolTip", "Use translation from Skeleton." ), FSlateIcon(), ActionRetargetingSkeleton);

		FUIAction ActionRetargetingLengthScale = FUIAction(FExecuteAction::CreateSP(this, &FSkeletonTreeBoneItem::SetBoneTranslationRetargetingMode, EBoneTranslationRetargetingMode::AnimationScaled));
		MenuBuilder.AddMenuEntry( Enum->GetDisplayNameTextByValue(EBoneTranslationRetargetingMode::AnimationScaled), LOCTEXT( "BoneTranslationRetargetingAnimationScaledToolTip", "Use translation from animation, scale length by Skeleton's proportions." ), FSlateIcon(), ActionRetargetingLengthScale);

		FUIAction ActionRetargetingAnimationRelative = FUIAction(FExecuteAction::CreateSP(this, &FSkeletonTreeBoneItem::SetBoneTranslationRetargetingMode, EBoneTranslationRetargetingMode::AnimationRelative));
		MenuBuilder.AddMenuEntry( Enum->GetDisplayNameTextByValue(EBoneTranslationRetargetingMode::AnimationRelative), LOCTEXT("BoneTranslationRetargetingAnimationRelativeToolTip", "Use relative translation from animation similar to an additive animation."), FSlateIcon(), ActionRetargetingAnimationRelative);

		FUIAction ActionRetargetingOrientAndScale = FUIAction(FExecuteAction::CreateSP(this, &FSkeletonTreeBoneItem::SetBoneTranslationRetargetingMode, EBoneTranslationRetargetingMode::OrientAndScale));
		MenuBuilder.AddMenuEntry(Enum->GetDisplayNameTextByValue(EBoneTranslationRetargetingMode::OrientAndScale), LOCTEXT("BoneTranslationRetargetingOrientAndScaleToolTip", "Orient And Scale Translation."), FSlateIcon(), ActionRetargetingOrientAndScale);

	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText FSkeletonTreeBoneItem::GetTranslationRetargetingModeMenuTitle() const
{
	const USkeleton& Skeleton = GetEditableSkeleton()->GetSkeleton();

	const int32 BoneIndex = Skeleton.GetReferenceSkeleton().FindBoneIndex( BoneName );
	if( BoneIndex != INDEX_NONE )
	{
		const EBoneTranslationRetargetingMode::Type RetargetingMode = Skeleton.GetBoneTranslationRetargetingMode(BoneIndex);
		UEnum* const Enum = StaticEnum<EBoneTranslationRetargetingMode::Type>();	
		if (Enum)
		{
			return Enum->GetDisplayNameTextByValue(RetargetingMode);
		}
	}

	return LOCTEXT("None", "None");
}

void FSkeletonTreeBoneItem::SetBoneTranslationRetargetingMode(EBoneTranslationRetargetingMode::Type NewRetargetingMode)
{
	GetEditableSkeleton()->SetBoneTranslationRetargetingMode(BoneName, NewRetargetingMode);
}

FSlateFontInfo FSkeletonTreeBoneItem::GetBoneTextFont() const
{
	if (!bRequiredBone)
	{
		return FAppStyle::GetWidgetStyle<FTextBlockStyle>("SkeletonTree.ItalicFont").Font;
	}
	else
	{
		return FAppStyle::GetWidgetStyle<FTextBlockStyle>("SkeletonTree.NormalFont").Font;
	}
}

void FSkeletonTreeBoneItem::CacheLODChange(UDebugSkelMeshComponent* PreviewComponent)
{
	bWeightedBone = false;
	bRequiredBone = false;
	if (PreviewComponent)
	{
		int32 BoneIndex = PreviewComponent->GetBoneIndex(BoneName);

		if (BoneIndex != INDEX_NONE)
		{
			if (IsBoneWeighted(BoneIndex, PreviewComponent))
			{
				//Bone is vertex weighted
				bWeightedBone = true;
			}
			if (IsBoneRequired(BoneIndex, PreviewComponent))
			{
				bRequiredBone = true;
			}
		}
	}
}

void FSkeletonTreeBoneItem::EnableBoneProxyTick(bool bEnable)
{
	BoneProxy->bIsTickable = bEnable;
}

FSlateColor FSkeletonTreeBoneItem::GetBoneTextColor(FIsSelected InIsSelected) const
{
	if (FilterResult == ESkeletonTreeFilterResult::ShownDescendant)
	{
		return FSlateColor(FLinearColor::Gray * 0.5f);
	}

	bool bIsSelected = InIsSelected.IsBound() ? InIsSelected.Execute() : false;
	if (bIsSelected)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundInverted");
	}
	else if (bRequiredBone && bWeightedBone)
	{
		return FSlateColor::UseForeground();
	}
	else
	{
		return FSlateColor::UseSubduedForeground();
	}
}

FReply FSkeletonTreeBoneItem::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		return FReply::Handled().BeginDragDrop( FBoneDragDropOp::New(GetEditableSkeleton(), BoneName ) );
	}

	return FReply::Unhandled();
}

FText FSkeletonTreeBoneItem::GetBoneToolTip()
{
	bool bIsMeshBone = false;
	bool bIsWeightedBone = false;
	bool bMeshExists = false;

	FText ToolTip;

	if (GetSkeletonTree()->GetPreviewScene().IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent();

		if (PreviewComponent && PreviewComponent->GetSkeletalMeshAsset())
		{
			bMeshExists = true;

			int32 BoneIndex = PreviewComponent->GetBoneIndex(BoneName);

			if (BoneIndex != INDEX_NONE)
			{
				bIsMeshBone = true;

				bIsWeightedBone = IsBoneWeighted(BoneIndex, PreviewComponent);
			}
		}
	}

	if ( !bMeshExists )
	{
		ToolTip = LOCTEXT( "BoneToolTipNoMeshAvailable", "This bone exists only on the skeleton as there is no current mesh set" );
	}
	else
	{
		if ( !bIsMeshBone )
		{
			ToolTip = LOCTEXT( "BoneToolTipSkeletonOnly", "This bone exists only on the skeleton, but not on the current mesh" );
		}
		else
		{
			if ( !bIsWeightedBone )
			{
				ToolTip = LOCTEXT( "BoneToolTipSkeletonAndMesh", "This bone is used by the current mesh, but has no vertices weighted against it" );
			}
			else
			{
				ToolTip = LOCTEXT( "BoneToolTipWeighted", "This bone (or one of its children) has vertices weighted against it" );
			}
		}
	}

	return ToolTip;
}

void FSkeletonTreeBoneItem::OnBeginBlendSliderMovement()
{
	if (bBlendSliderStartedTransaction == false)
	{
		bBlendSliderStartedTransaction = true;
		GEditor->BeginTransaction(LOCTEXT("BlendSliderTransation", "Modify Blend Profile Value"));

		const FName& BlendProfileName = GetSkeletonTree()->GetSelectedBlendProfile()->GetFName();
		UBlendProfile* BlendProfile = GetEditableSkeleton()->GetBlendProfile(BlendProfileName);

		if (BlendProfile)
		{
			BlendProfile->SetFlags(RF_Transactional);
			BlendProfile->Modify();
		}
	}
}
void FSkeletonTreeBoneItem::OnEndBlendSliderMovement(float NewValue)
{
	if (bBlendSliderStartedTransaction)
	{
		GEditor->EndTransaction();
		bBlendSliderStartedTransaction = false;
	}
}

void FSkeletonTreeBoneItem::OnBlendSliderCommitted(float NewValue, ETextCommit::Type CommitType)
{
	FName BlendProfileName = GetSkeletonTree()->GetSelectedBlendProfile()->GetFName();
	UBlendProfile* BlendProfile = GetEditableSkeleton()->GetBlendProfile(BlendProfileName);

	if (BlendProfile)
	{
		FScopedTransaction Transaction(LOCTEXT("SetBlendProfileValue", "Set Blend Profile Value"));
		BlendProfile->SetFlags(RF_Transactional);
		BlendProfile->Modify();

		BlendProfile->SetBoneBlendScale(BoneName, NewValue, false, true);
	}
}

void FSkeletonTreeBoneItem::OnBlendSliderChanged(float NewValue)
{
	const FName& BlendProfileName = GetSkeletonTree()->GetSelectedBlendProfile()->GetFName();
	UBlendProfile* BlendProfile = GetEditableSkeleton()->GetBlendProfile(BlendProfileName);
	
	if (BlendProfile)
	{
		BlendProfile->SetBoneBlendScale(BoneName, NewValue, false, true);
	}
}

void FSkeletonTreeBoneItem::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSocketDragDropOp> DragConnectionOp = DragDropEvent.GetOperationAs<FSocketDragDropOp>();

	// Is someone dragging a socket onto a bone?
	if (DragConnectionOp.IsValid())
	{
		if (BoneName != DragConnectionOp->GetSocketInfo().Socket->BoneName)
		{
			// The socket can be dropped here if we're a bone and NOT the socket's existing parent
			DragConnectionOp->SetIcon( FAppStyle::GetBrush( TEXT( "Graph.ConnectorFeedback.Ok" ) ) );
		}
		else if (DragConnectionOp->IsAltDrag())
		{
			// For Alt-Drag, dropping onto the existing parent is fine, as we're going to copy, not move the socket
			DragConnectionOp->SetIcon( FAppStyle::GetBrush( TEXT( "Graph.ConnectorFeedback.Ok" ) ) );
		}
	}
}

void FSkeletonTreeBoneItem::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSocketDragDropOp> DragConnectionOp = DragDropEvent.GetOperationAs<FSocketDragDropOp>();
	if (DragConnectionOp.IsValid())
	{
		// Reset the drag/drop icon when leaving this row
		DragConnectionOp->SetIcon( FAppStyle::GetBrush( TEXT( "Graph.ConnectorFeedback.Error" ) ) );
	}
}

FReply FSkeletonTreeBoneItem::HandleDrop(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSocketDragDropOp> DragConnectionOp = DragDropEvent.GetOperationAs<FSocketDragDropOp>();
	if (DragConnectionOp.IsValid())
	{
		FSelectedSocketInfo SocketInfo = DragConnectionOp->GetSocketInfo();

		if (DragConnectionOp->IsAltDrag())
		{
			// In an alt-drag, the socket can be dropped on any bone
			// (including its existing parent) to create a uniquely named copy
			GetSkeletonTree()->DuplicateAndSelectSocket( SocketInfo, BoneName);
		}
		else if (BoneName != SocketInfo.Socket->BoneName)
		{
			// The socket can be dropped here if we're a bone and NOT the socket's existing parent
			USkeletalMesh* SkeletalMesh = GetSkeletonTree()->GetPreviewScene().IsValid() ? ToRawPtr(GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;
			const bool bKeepAbsoluteLocation = DragConnectionOp->IsShiftDrag();
			GetEditableSkeleton()->SetSocketParent(SocketInfo.Socket->SocketName, BoneName, SkeletalMesh, bKeepAbsoluteLocation);

			return FReply::Handled();
		}
	}
	else
	{
		TSharedPtr<FAssetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
		if (DragDropOp.IsValid())
		{
			//Do we have some assets to attach?
			if (DragDropOp->HasAssets())
			{
				GetSkeletonTree()->AttachAssets(SharedThis(this), DragDropOp->GetAssets());
			}
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool FSkeletonTreeBoneItem::IsBoneWeighted(int32 MeshBoneIndex, UDebugSkelMeshComponent* PreviewComponent)
{
	// MeshBoneIndex must be an index into the mesh's skeleton, *not* the source skeleton!!!
	if (MeshBoneIndex == INDEX_NONE)
	{
		// If we get an invalid index, we are done here
		return false; 
	}

	if (!PreviewComponent || !PreviewComponent->GetSkeletalMeshAsset() || !PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering() || !PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering()->LODRenderData.Num())
	{
		// If there's no mesh, then this bone can't possibly be weighted!
		return false;
	}

	//Get current LOD
	const int32 LODIndex = FMath::Clamp(PreviewComponent->GetPredictedLODLevel(), 0, PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering()->LODRenderData.Num() - 1);
	FSkeletalMeshLODRenderData& LODData = PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering()->LODRenderData[LODIndex];

	//Check whether the bone is vertex weighted
	int32 Index = LODData.ActiveBoneIndices.Find(IntCastChecked<FBoneIndexType>(MeshBoneIndex));

	return Index != INDEX_NONE;
}

bool FSkeletonTreeBoneItem::IsBoneRequired(int32 MeshBoneIndex, UDebugSkelMeshComponent* PreviewComponent)
{
	// MeshBoneIndex must be an index into the mesh's skeleton, *not* the source skeleton!!!

	if (!PreviewComponent || !PreviewComponent->GetSkeletalMeshAsset() || !PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering() || !PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering()->LODRenderData.Num())
	{
		// If there's no mesh, then this bone can't possibly be weighted!
		return false;
	}

	//Check whether the bone is vertex weighted or required by physics or other forced requirements (see USkeletalMeshComponent::ComputeRequiredBones)
	const int32 Index = PreviewComponent->RequiredBones.Find(IntCastChecked<FBoneIndexType>(MeshBoneIndex));

	return Index != INDEX_NONE;
}

void FSkeletonTreeBoneItem::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(BoneProxy);
}

#undef LOCTEXT_NAMESPACE