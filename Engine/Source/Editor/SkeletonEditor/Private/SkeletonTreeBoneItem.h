// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "ISkeletonTreeItem.h"
#include "SkeletonTreeItem.h"
#include "Animation/Skeleton.h"
#include "BoneProxy.h"
#include "UObject/GCObject.h"

class SComboButton;
class SWidget;
class UDebugSkelMeshComponent;
struct EVisibility;
struct FSlateBrush;
struct FSlateColor;
struct FSlateFontInfo;

class FSkeletonTreeBoneItem : public FSkeletonTreeItem, public FGCObject
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreeBoneItem, FSkeletonTreeItem)

	FSkeletonTreeBoneItem(const FName& InBoneName, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	/** ISkeletonTreeItem interface */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override;
	virtual FName GetRowItemName() const override { return BoneName; }
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) override;
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual UObject* GetObject() const override { return BoneProxy; }

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSkeletonTreeBoneItem");
	}

	/** Check to see if the specified bone is weighted in the specified component */
	bool IsBoneWeighted(int32 MeshBoneIndex, UDebugSkelMeshComponent* PreviewComponent);

	/** Check to see if the specified bone is required in the specified component */
	bool IsBoneRequired(int32 MeshBoneIndex, UDebugSkelMeshComponent* PreviewComponent);

	/** Cache to use Bold Font or not */
	void CacheLODChange(UDebugSkelMeshComponent* PreviewComponent);

	/** Enable and disable the bone proxy ticking */
	void EnableBoneProxyTick(bool bEnable);

private:
	/** Gets the font for displaying bone text in the skeletal tree */
	FSlateFontInfo GetBoneTextFont() const;

	/** Get the text color based on bone part of skeleton or part of mesh */
	FSlateColor GetBoneTextColor(FIsSelected InIsSelected) const;

	/** Brush of the icon */
	const FSlateBrush* GetLODIcon() const;

	/** Function that returns the current tooltip for this bone, depending on how it's used by the mesh */
	FText GetBoneToolTip();

	/** Vary the color of the combo based on hover state */
	FSlateColor GetRetargetingComboButtonForegroundColor() const;

	/** Create menu for Bone Translation Retargeting Mode. */
	TSharedRef< SWidget > CreateBoneTranslationRetargetingModeMenu();

	/** Get Title for Bone Translation Retargeting Mode menu. */
	FText GetTranslationRetargetingModeMenuTitle() const;

	/** Callback from a slider widget if the text entry is used */
	void OnBlendSliderCommitted(float NewValue, ETextCommit::Type CommitType);

	/** Callback from a slider widget if the slider is used */
	void OnBlendSliderChanged(float NewValue);

	/** Callback from a slider widget when the user begins sliding */
	void OnBeginBlendSliderMovement();

	/** Callback from a slider widget when the user has finished sliding */
	void OnEndBlendSliderMovement(float NewValue);

	/** Set Translation Retargeting Mode for this bone. */
	void SetBoneTranslationRetargetingMode(EBoneTranslationRetargetingMode::Type NewRetargetingMode);

	/** Get the current Blend Scale for this bone */
	float GetBoneBlendProfileScale() const;

	/** Get the max slider value supported by the selected blend profile's mode*/
	TOptional<float> GetBlendProfileMaxSliderValue() const;

	/** Get the min slider value supported by the selected blend profile's mode*/
	TOptional<float> GetBlendProfileMinSliderValue() const;

	/** Used to hide the Bone Profile row widget if there is no current Blend Profile */
	EVisibility GetBoneBlendProfileVisibility() const;

private:
	/** Bone proxy object */
	TObjectPtr<UBoneProxy> BoneProxy;

	/** The actual bone data that we create Slate widgets to display */
	FName BoneName;

	/** Weighted bone */
	bool bWeightedBone;

	/** Required bone */
	bool bRequiredBone;

	/** Reference to the retargeting combo button */
	TSharedPtr<SComboButton> RetargetingComboButton;

	/** True if the user is in a transaction when moving the blend profile slider */
	bool bBlendSliderStartedTransaction;
};
