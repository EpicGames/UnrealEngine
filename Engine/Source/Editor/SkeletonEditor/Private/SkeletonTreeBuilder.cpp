// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeBuilder.h"
#include "IPersonaPreviewScene.h"
#include "SkeletonTreeBoneItem.h"
#include "SkeletonTreeSocketItem.h"
#include "SkeletonTreeAttachedAssetItem.h"
#include "SkeletonTreeVirtualBoneItem.h"
#include "Animation/DebugSkelMeshComponent.h"

#define LOCTEXT_NAMESPACE "SkeletonTreeBuilder"

void FSkeletonTreeBuilderOutput::Add(const TSharedPtr<class ISkeletonTreeItem>& InItem, const FName& InParentName, const FName& InParentType, bool bAddToHead)
{
	Add(InItem, InParentName, TArray<FName, TInlineAllocator<1>>({ InParentType }), bAddToHead);
}

void FSkeletonTreeBuilderOutput::Add(const TSharedPtr<class ISkeletonTreeItem>& InItem, const FName& InParentName, TArrayView<const FName> InParentTypes, bool bAddToHead)
{
	TSharedPtr<ISkeletonTreeItem> ParentItem = Find(InParentName, InParentTypes);
	if (ParentItem.IsValid())
	{
		InItem->SetParent(ParentItem);

		if (bAddToHead)
		{
			ParentItem->GetChildren().Insert(InItem, 0);
		}
		else
		{
			ParentItem->GetChildren().Add(InItem);
		}
	}
	else
	{
		if (bAddToHead)
		{
			Items.Insert(InItem, 0);
		}
		else
		{
			Items.Add(InItem);
		}
	}

	LinearItems.Add(InItem);
}

TSharedPtr<class ISkeletonTreeItem> FSkeletonTreeBuilderOutput::Find(const FName& InName, const FName& InType) const
{
	return Find(InName, TArray<FName, TInlineAllocator<1>>({ InType }));
}

TSharedPtr<class ISkeletonTreeItem> FSkeletonTreeBuilderOutput::Find(const FName& InName, TArrayView<const FName> InTypes) const
{
	for (const TSharedPtr<ISkeletonTreeItem>& Item : LinearItems)
	{
		bool bPassesType = (InTypes.Num() == 0);
		for (const FName& TypeName : InTypes)
		{
			if (Item->IsOfTypeByName(TypeName))
			{
				bPassesType = true;
				break;
			}
		}

		if (bPassesType && Item->GetAttachName() == InName)
		{
			return Item;
		}
	}

	return nullptr;
}

FSkeletonTreeBuilder::FSkeletonTreeBuilder(const FSkeletonTreeBuilderArgs& InBuilderArgs)
	: BuilderArgs(InBuilderArgs)
{
}

void FSkeletonTreeBuilder::Initialize(const TSharedRef<class ISkeletonTree>& InSkeletonTree, const TSharedPtr<class IPersonaPreviewScene>& InPreviewScene, FOnFilterSkeletonTreeItem InOnFilterSkeletonTreeItem)
{
	SkeletonTreePtr = InSkeletonTree;
	EditableSkeletonPtr = InSkeletonTree->GetEditableSkeleton();
	PreviewScenePtr = InPreviewScene;
	OnFilterSkeletonTreeItem = InOnFilterSkeletonTreeItem;
}

void FSkeletonTreeBuilder::Build(FSkeletonTreeBuilderOutput& Output)
{
	if(BuilderArgs.bShowBones)
	{
		AddBones(Output);
	}

	if (BuilderArgs.bShowVirtualBones)
	{
		AddVirtualBones(Output);
	}

	if (BuilderArgs.bShowSockets)
	{
		AddSockets(Output);
	}
	
	if (BuilderArgs.bShowAttachedAssets)
	{
		AddAttachedAssets(Output);
	}
}

void FSkeletonTreeBuilder::Filter(const FSkeletonTreeFilterArgs& InArgs, const TArray<TSharedPtr<ISkeletonTreeItem>>& InItems, TArray<TSharedPtr<ISkeletonTreeItem>>& OutFilteredItems)
{
	OutFilteredItems.Empty();

	for (const TSharedPtr<ISkeletonTreeItem>& Item : InItems)
	{
		if (InArgs.TextFilter.IsValid() && InArgs.bFlattenHierarchyOnFilter)
		{
			FilterRecursive(InArgs, Item, OutFilteredItems);
		}
		else
		{
			ESkeletonTreeFilterResult FilterResult = FilterRecursive(InArgs, Item, OutFilteredItems);
			if (FilterResult != ESkeletonTreeFilterResult::Hidden)
			{
				OutFilteredItems.Add(Item);
			}

			Item->SetFilterResult(FilterResult);
		}
	}
}

ESkeletonTreeFilterResult FSkeletonTreeBuilder::FilterRecursive(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<ISkeletonTreeItem>& InItem, TArray<TSharedPtr<ISkeletonTreeItem>>& OutFilteredItems)
{
	ESkeletonTreeFilterResult FilterResult = ESkeletonTreeFilterResult::Shown;

	InItem->GetFilteredChildren().Empty();

	if (InArgs.TextFilter.IsValid() && InArgs.bFlattenHierarchyOnFilter)
	{
		FilterResult = FilterItem(InArgs, InItem);
		InItem->SetFilterResult(FilterResult);

		if (FilterResult != ESkeletonTreeFilterResult::Hidden)
		{
			OutFilteredItems.Add(InItem);
		}

		for (const TSharedPtr<ISkeletonTreeItem>& Item : InItem->GetChildren())
		{
			FilterRecursive(InArgs, Item, OutFilteredItems);
		}
	}
	else
	{
		// check to see if we have any children that pass the filter
		ESkeletonTreeFilterResult DescendantsFilterResult = ESkeletonTreeFilterResult::Hidden;
		for (const TSharedPtr<ISkeletonTreeItem>& Item : InItem->GetChildren())
		{
			ESkeletonTreeFilterResult ChildResult = FilterRecursive(InArgs, Item, OutFilteredItems);
			if (ChildResult != ESkeletonTreeFilterResult::Hidden)
			{
				InItem->GetFilteredChildren().Add(Item);
			}
			if (ChildResult > DescendantsFilterResult)
			{
				DescendantsFilterResult = ChildResult;
			}
		}

		FilterResult = FilterItem(InArgs, InItem);
		if (DescendantsFilterResult > FilterResult)
		{
			FilterResult = ESkeletonTreeFilterResult::ShownDescendant;
		}

		InItem->SetFilterResult(FilterResult);
	}

	return FilterResult;
}

ESkeletonTreeFilterResult FSkeletonTreeBuilder::FilterItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem)
{
	return OnFilterSkeletonTreeItem.Execute(InArgs, InItem);
}

bool FSkeletonTreeBuilder::IsShowingBones() const
{
	return BuilderArgs.bShowBones;
}

bool FSkeletonTreeBuilder::IsShowingSockets() const
{
	return BuilderArgs.bShowSockets;
}

bool FSkeletonTreeBuilder::IsShowingAttachedAssets() const
{
	return BuilderArgs.bShowAttachedAssets;
}

void FSkeletonTreeBuilder::AddBones(FSkeletonTreeBuilderOutput& Output)
{
	TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin();
	if (!EditableSkeleton || !EditableSkeleton->IsSkeletonValid())
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton tree builder - skeleton is invalid");
		return;
	}

	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
	const FReferenceSkeleton& RefSkeleton = Skeleton.GetReferenceSkeleton();

	struct FBoneInfo
	{
		FBoneInfo(const FName& InBoneName, const FName& InParentName, int32 InDepth)
			: BoneName(InBoneName)
			, ParentName(InParentName)
			, Depth(InDepth)
		{
			SortString = BoneName.ToString();
			SortNumber = 0;
			SortLength = SortString.Len();

			// Split the bone name into string prefix and numeric suffix for sorting (different from FName to support leading zeros in the numeric suffix)
			int32 Index = SortLength - 1;
			for (int32 PlaceValue = 1; Index >= 0 && FChar::IsDigit(SortString[Index]); --Index, PlaceValue *= 10)
			{
				SortNumber += static_cast<int32>(SortString[Index] - '0') * PlaceValue;
			}
			SortString.LeftInline(Index + 1, EAllowShrinking::No);
		}

		bool operator<(const FBoneInfo& RHS)
		{
			// Sort parents before children
			if (Depth != RHS.Depth)
			{
				return Depth < RHS.Depth;
			}

			// Sort alphabetically by string prefix
			if (int32 SplitNameComparison = SortString.Compare(RHS.SortString))
			{
				return SplitNameComparison < 0;
			}

			// Sort by number if the string prefixes match
			if (SortNumber != RHS.SortNumber)
			{
				return SortNumber < RHS.SortNumber;
			}

			// Sort by length to give us the equivalent to alphabetical sorting if the numbers match (which gives us the following sort order: bone_, bone_0, bone_00, bone_000, bone_001, bone_01, bone_1, etc)
			return (SortNumber == 0) ? SortLength < RHS.SortLength : SortLength > RHS.SortLength;
		}

		FName BoneName = NAME_None;
		FName ParentName = NAME_None;
		int32 Depth = 0;

		FString SortString;
		int32 SortNumber = 0;
		int32 SortLength = 0;
	};

	TArray<FBoneInfo> Bones;
	Bones.Reserve(RefSkeleton.GetRawBoneNum());

	// Gather the bones from the skeleton
	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
	{
		const FName& BoneName = RefSkeleton.GetBoneName(BoneIndex);

		FName ParentName = NAME_None;
		int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
		int32 Depth = 0;
		if (ParentIndex != INDEX_NONE)
		{
			ParentName = RefSkeleton.GetBoneName(ParentIndex);
			Depth = Bones[ParentIndex].Depth + 1;
		}

		Bones.Emplace(BoneName, ParentName, Depth);
	}

	// Sort the bones lexically (and also by depth in order to maintain the invariant of parents before children)
	Algo::Sort(Bones);

	// Add the sorted bones to the skeleton tree
	for (const FBoneInfo& Bone : Bones)
	{
		TSharedRef<ISkeletonTreeItem> DisplayBone = CreateBoneTreeItem(Bone.BoneName);
		Output.Add(DisplayBone, Bone.ParentName, FSkeletonTreeBoneItem::GetTypeId());
	}
}

void FSkeletonTreeBuilder::AddSockets(FSkeletonTreeBuilderOutput& Output)
{
	TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin();
	if (!EditableSkeleton || !EditableSkeleton->IsSkeletonValid())
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton tree builder - skeleton is invalid");
		return;
	}
	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();

	// Add the sockets for the skeleton
	AddSocketsFromData(Skeleton.Sockets, ESocketParentType::Skeleton, Output);

	// Add the sockets for the mesh
	if (PreviewScenePtr.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		if (USkeletalMesh* const SkeletalMesh = PreviewMeshComponent->GetSkeletalMeshAsset())
		{
			AddSocketsFromData(SkeletalMesh->GetMeshOnlySocketList(), ESocketParentType::Mesh, Output);
		}
	}
}

void FSkeletonTreeBuilder::AddAttachedAssets(FSkeletonTreeBuilderOutput& Output)
{
	TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin();
	if (!EditableSkeleton || !EditableSkeleton->IsSkeletonValid())
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton tree builder - skeleton is invalid");
		return;
	}

	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();

	// Mesh attached items...
	if (PreviewScenePtr.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		if (USkeletalMesh* const SkeletalMesh = PreviewMeshComponent->GetSkeletalMeshAsset())
		{
			AddAttachedAssetContainer(SkeletalMesh->GetPreviewAttachedAssetContainer(), Output);
		}
	}

	// ...skeleton attached items
	AddAttachedAssetContainer(Skeleton.PreviewAttachedAssetContainer, Output);
}

void FSkeletonTreeBuilder::AddSocketsFromData(const TArray< USkeletalMeshSocket* >& SocketArray, ESocketParentType ParentType, FSkeletonTreeBuilderOutput& Output)
{
	TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin();
	if (!EditableSkeleton || !EditableSkeleton->IsSkeletonValid())
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton tree builder - skeleton is invalid");
		return;
	}

	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();

	for (auto SocketIt = SocketArray.CreateConstIterator(); SocketIt; ++SocketIt)
	{
		USkeletalMeshSocket* Socket = *(SocketIt);

		bool bIsCustomized = false;

		if (ParentType == ESocketParentType::Mesh)
		{
			bIsCustomized = EditableSkeleton->DoesSocketAlreadyExist(nullptr, FText::FromName(Socket->SocketName), ESocketParentType::Skeleton, nullptr);
		}
		else
		{
			if (PreviewScenePtr.IsValid())
			{
				UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
				if (USkeletalMesh* const SkeletalMesh = PreviewMeshComponent->GetSkeletalMeshAsset())
				{
					bIsCustomized = EditableSkeleton->DoesSocketAlreadyExist(nullptr, FText::FromName(Socket->SocketName), ESocketParentType::Mesh, SkeletalMesh);
				}
			}
		}

		Output.Add(CreateSocketTreeItem(Socket, ParentType, bIsCustomized), Socket->BoneName, { FSkeletonTreeBoneItem::GetTypeId(), FSkeletonTreeVirtualBoneItem::GetTypeId() }, /*bAddToHead*/ true);
	}
}

void FSkeletonTreeBuilder::AddAttachedAssetContainer(const FPreviewAssetAttachContainer& AttachedObjects, FSkeletonTreeBuilderOutput& Output)
{
	for (auto Iter = AttachedObjects.CreateConstIterator(); Iter; ++Iter)
	{
		const FPreviewAttachedObjectPair& Pair = (*Iter);

		Output.Add(CreateAttachedAssetTreeItem(Pair.GetAttachedObject(), Pair.AttachedTo), Pair.AttachedTo, { FSkeletonTreeBoneItem::GetTypeId(), FSkeletonTreeSocketItem::GetTypeId() }, /*bAddToHead*/ true);
	}
}

void FSkeletonTreeBuilder::AddVirtualBones(FSkeletonTreeBuilderOutput& Output)
{
	TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin();
	if (!EditableSkeleton || !EditableSkeleton->IsSkeletonValid())
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton tree builder - skeleton is invalid");
		return;
	}

	const TArray<FVirtualBone>& VirtualBones = EditableSkeleton->GetSkeleton().GetVirtualBones();
	for (const FVirtualBone& VirtualBone : VirtualBones)
	{
		Output.Add(CreateVirtualBoneTreeItem(VirtualBone.VirtualBoneName), VirtualBone.SourceBoneName, { FSkeletonTreeBoneItem::GetTypeId(), FSkeletonTreeVirtualBoneItem::GetTypeId() }, /*bAddToHead*/ true);
	}
}

TSharedRef<class ISkeletonTreeItem> FSkeletonTreeBuilder::CreateBoneTreeItem(const FName& InBoneName)
{
	return MakeShareable(new FSkeletonTreeBoneItem(InBoneName, SkeletonTreePtr.Pin().ToSharedRef()));
}

TSharedRef<class ISkeletonTreeItem> FSkeletonTreeBuilder::CreateSocketTreeItem(USkeletalMeshSocket* InSocket, ESocketParentType InParentType, bool bInIsCustomized)
{
	return MakeShareable(new FSkeletonTreeSocketItem(InSocket, InParentType, bInIsCustomized, SkeletonTreePtr.Pin().ToSharedRef()));
}

TSharedRef<class ISkeletonTreeItem> FSkeletonTreeBuilder::CreateAttachedAssetTreeItem(UObject* InAsset, const FName& InAttachedTo)
{
	return MakeShareable(new FSkeletonTreeAttachedAssetItem(InAsset, InAttachedTo, SkeletonTreePtr.Pin().ToSharedRef()));
}

TSharedRef<class ISkeletonTreeItem> FSkeletonTreeBuilder::CreateVirtualBoneTreeItem(const FName& InBoneName)
{
	return MakeShareable(new FSkeletonTreeVirtualBoneItem(InBoneName, SkeletonTreePtr.Pin().ToSharedRef()));
}

#undef LOCTEXT_NAMESPACE