// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "SkeletonTreeCommands"

void FSkeletonTreeCommands::RegisterCommands()
{
	UI_COMMAND( ShowAllBones, "Show All Bones", "Show every bone in the skeleton", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowMeshBones, "Show Mesh Bones", "Show bones that are used in the mesh", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowLODBones, "Show LOD Bones", "Show bones that are used in the LOD displayed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowWeightedBones, "Show Weighted Bones", "Show bones that have vertices weighted to them", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( HideBones, "Hide Bones", "Hides all bones (sockets and attached assets will still be listed)", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowRetargeting, "Show Retargeting Options", "Show retargeting options for each bone.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ShowDebugVisualization, "Show Debug Visualization Options", "Show debug visualization options for objects in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( FilteringFlattensHierarchy, "Filtering Flattens Hierarchy", "Whether to keep the hierarchy or flatten it when searching for tree items", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( HideParentsWhenFiltering, "Hide Parents When Filtering", "Whether to show parent items grayed out, or hide them entirely when filtering", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ShowBoneIndex, "Show Bone Index", "Show the index of the bone as stored in the reference skeleton", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND( CopyBoneNames, "Copy Bone Names", "Copy selected bone names to clipboard", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ResetBoneTransforms, "Reset Bone Transforms", "Reset the transforms of the selected bones", EUserInterfaceActionType::Button, FInputChord(EKeys::I, EModifierKey::Control) );
	UI_COMMAND( ResetAllBonesTransforms, "Reset All Bones Transforms", "Reset the transforms of all the bones", EUserInterfaceActionType::Button, FInputChord(EKeys::I, EModifierKey::Control | EModifierKey::Shift) );
	UI_COMMAND( CopySockets, "Copy Selected Sockets", "Copy selected sockets to clipboard", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::C ) );
	UI_COMMAND( PasteSockets, "Paste Sockets", "Paste sockets from clipboard (try to add them to the original copy bone or root if that bone is not available)", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::V ) );
	UI_COMMAND( PasteSocketsToSelectedBone, "Paste Sockets To Selected Bone", "Paste sockets from clipboard to selected bone", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::V));

	UI_COMMAND( AddSocket, "Add Socket", "Add a socket to this bone in the skeleton (disabled when socket filter is set to \"Mesh Sockets\" or \"Sockets Hidden\" mode)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateMeshSocket, "Create Mesh Socket", "Duplicate this socket from skeleton to the current mesh and modify the socket data for it", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RemoveMeshSocket, "Remove Mesh Socket", "Remove duplicated version of this socket for the current mesh (reverts to the socket in the skeleton)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( PromoteSocketToSkeleton, "Promote Socket To Skeleton", "Makes this socket available for all meshes that use the same skeleton (copies the socket from this mesh to the skeleton)", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND(DeleteSelectedRows, "Delete", "Delete all selected items in the tree", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));

	UI_COMMAND( ShowActiveSockets, "Show Active Sockets", "Show mesh and skeleton sockets, hiding the skeleton sockets that have a customized mesh socket", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowAllSockets, "Show All Sockets", "Show all sockets that are in the mesh and skeleton", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowMeshSockets, "Show Mesh Sockets", "Show sockets that are in the mesh only", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowSkeletonSockets, "Show Skeleton Sockets", "Show sockets that are in the skeleton only", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( HideSockets, "Hide Sockets", "Show no sockets", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND( FocusCamera, "Focus Camera", "Focus the camera on the current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::F) );

	UI_COMMAND( CreateTimeBlendProfile, "Add Time Blend Profile", "Add a new time blend profile for this skeleton", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateWeightBlendProfile, "Add Weight Blend Profile", "Add a new weight blend profile for this skeleton", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateBlendMask, "Add Blend Mask", "Add a new blend mask for this skeleton", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DeleteCurrentBlendProfile, "Delete the Current Blend Profile", "Delete the active blend profile for this skeleton", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RenameBlendProfile, "Rename the Current Blend Profile", "Rename the active blend profile for this skeleton", EUserInterfaceActionType::Button, FInputChord());

}

#undef LOCTEXT_NAMESPACE
