// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Class containing commands for skeleton tree actions
 */
class FSkeletonTreeCommands : public TCommands<FSkeletonTreeCommands>
{
public:
	FSkeletonTreeCommands() 
		: TCommands<FSkeletonTreeCommands>
		(
			TEXT("SkeletonTree"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "SkelTree", "Skeleton Tree"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{

	}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Show all bones in the tree */
	TSharedPtr< FUICommandInfo > ShowAllBones;

	/** Show only bones that are present in the current mesh */
	TSharedPtr< FUICommandInfo > ShowMeshBones;

	/** Show only bones that are present in the current LOD */
	TSharedPtr< FUICommandInfo > ShowLODBones;

	/** Show only bones that have weighted vertices */
	TSharedPtr< FUICommandInfo > ShowWeightedBones;

	/** Hide all bones */
	TSharedPtr< FUICommandInfo > HideBones;

	/** Show retargeting options */
	TSharedPtr< FUICommandInfo > ShowRetargeting;

	/** Show debug visualization options */
	TSharedPtr< FUICommandInfo > ShowDebugVisualization;

	/** Flatten hierarchy on filter */
	TSharedPtr< FUICommandInfo > FilteringFlattensHierarchy;

	/** Hide parents on filter */
	TSharedPtr< FUICommandInfo > HideParentsWhenFiltering;

	/** Show bone numbers */
	TSharedPtr< FUICommandInfo > ShowBoneIndex;

	/** Add a socket to the skeleton */
	TSharedPtr< FUICommandInfo > AddSocket;

	/** Customize a socket for a mesh */
	TSharedPtr< FUICommandInfo > CreateMeshSocket;

	/** Remove customization for a socket (actually just deletes the mesh socket) */
	TSharedPtr< FUICommandInfo > RemoveMeshSocket;

	/** Promotes a mesh-only socket to the skeleton */
	TSharedPtr< FUICommandInfo > PromoteSocketToSkeleton;

	/** Delete selected rows (deletes any sockets or assets selected in the tree */
	TSharedPtr< FUICommandInfo > DeleteSelectedRows;

	/** Show active sockets */
	TSharedPtr< FUICommandInfo > ShowActiveSockets;

	/** Show skeletal mesh sockets */
	TSharedPtr< FUICommandInfo > ShowMeshSockets;

	/** Show skeleton sockets */
	TSharedPtr< FUICommandInfo > ShowSkeletonSockets;

	/** Show all sockets */
	TSharedPtr< FUICommandInfo > ShowAllSockets;

	/** Hide all sockets */
	TSharedPtr< FUICommandInfo > HideSockets;

	/** Copy bone names */
	TSharedPtr< FUICommandInfo > CopyBoneNames;

	/** Reset selected bones transforms */
	TSharedPtr< FUICommandInfo > ResetBoneTransforms;

	/** Reset all bones transforms */
	TSharedPtr< FUICommandInfo > ResetAllBonesTransforms;

	/** Copy sockets to clipboard */
	TSharedPtr< FUICommandInfo > CopySockets;

	/** Paste sockets from clipboard */
	TSharedPtr< FUICommandInfo > PasteSockets;

	/** Paste sockets from clipboard */
	TSharedPtr< FUICommandInfo > PasteSocketsToSelectedBone;

	/** Focus the camera on the current selection */
	TSharedPtr< FUICommandInfo > FocusCamera;

	/** Create a new blend profile with time mode*/
	TSharedPtr< FUICommandInfo > CreateTimeBlendProfile;

	/** Create a new blend profile with weight mode*/
	TSharedPtr< FUICommandInfo > CreateWeightBlendProfile;

	/** Create a new blend mask */
	TSharedPtr< FUICommandInfo > CreateBlendMask;

	/** Remove the currently active blend profile */
	TSharedPtr< FUICommandInfo > DeleteCurrentBlendProfile;

	/** Rename an existing BlendProfile */
	TSharedPtr< FUICommandInfo > RenameBlendProfile;
};
