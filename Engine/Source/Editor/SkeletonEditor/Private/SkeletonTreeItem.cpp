// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeItem.h"

#include "Internationalization/Text.h"
#include "SSkeletonTreeRow.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;

TSharedRef<ITableRow> FSkeletonTreeItem::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, const TAttribute<FText>& InFilterText)
{
	return SNew(SSkeletonTreeRow, InOwnerTable)
		.FilterText(InFilterText)
		.Item(SharedThis(this))
		.OnDraggingItem(this, &FSkeletonTreeItem::OnDragDetected);
}
