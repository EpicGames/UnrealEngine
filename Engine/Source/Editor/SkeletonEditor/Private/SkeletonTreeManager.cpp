// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeManager.h"
#include "EditableSkeleton.h"
#include "ISkeletonTree.h"

FSkeletonTreeManager& FSkeletonTreeManager::Get()
{
	static FSkeletonTreeManager TheManager;
	return TheManager;
}

TSharedRef<ISkeletonTree> FSkeletonTreeManager::CreateSkeletonTree(const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs)
{
	TSharedPtr<ISkeletonTree> SkeletonTree = StaticCastSharedRef<FEditableSkeleton>(InEditableSkeleton)->CreateSkeletonTree(InSkeletonTreeArgs);

	// compact skeletons that are no longer being edited
	bool bRemoved = false;
	do
	{
		bRemoved = false;
		for (auto Iter = EditableSkeletons.CreateIterator(); Iter; ++Iter)
		{
			TSharedPtr<FEditableSkeleton> CompactionCandidate = Iter.Value().Pin();
			if (!CompactionCandidate.IsValid() || !CompactionCandidate->IsEdited())
			{
				Iter.RemoveCurrent();
				bRemoved = true;
				break;
			}
		}
	}
	while (bRemoved);

	return SkeletonTree.ToSharedRef();
}

TSharedRef<ISkeletonTree> FSkeletonTreeManager::CreateSkeletonTree(class USkeleton* InSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs)
{
	return CreateSkeletonTree(CreateEditableSkeleton(InSkeleton), InSkeletonTreeArgs);
}

TSharedRef<FEditableSkeleton> FSkeletonTreeManager::CreateEditableSkeleton(class USkeleton* InSkeleton)
{
	check(InSkeleton);

	TWeakPtr<FEditableSkeleton>* EditableSkeletonPtr = EditableSkeletons.Find(InSkeleton);
	if (EditableSkeletonPtr == nullptr || !(*EditableSkeletonPtr).IsValid())
	{
		TSharedRef<FEditableSkeleton> NewEditableSkeleton = MakeShareable(new FEditableSkeleton(InSkeleton));
		EditableSkeletons.Add(InSkeleton, NewEditableSkeleton);
		return NewEditableSkeleton;
	}
	else
	{
		return (*EditableSkeletonPtr).Pin().ToSharedRef();
	}
}
