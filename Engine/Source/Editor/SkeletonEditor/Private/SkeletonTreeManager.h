// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISkeletonTree;
class FEditableSkeleton;
class IEditableSkeleton;
class USkeleton;
struct FSkeletonTreeArgs;

/** Central registry of skeleton trees */
class FSkeletonTreeManager
{
public:
	/** Singleton access */
	static FSkeletonTreeManager& Get();

	/** Create a skeleton tree for the given an already-existing editable skeleton */
	TSharedRef<ISkeletonTree> CreateSkeletonTree(const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs);

	/** Create a skeleton tree for the requested skeleton */
	TSharedRef<ISkeletonTree> CreateSkeletonTree(USkeleton* InSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs);

	/** Edit a USkeleton via FEditableSkeleton */
	TSharedRef<FEditableSkeleton> CreateEditableSkeleton(USkeleton* InSkeleton);

private:
	/** Hidden constructor */
	FSkeletonTreeManager() {}

private:
	/** Map from skeletons to editable skeletons */
	TMap<USkeleton*, TWeakPtr<FEditableSkeleton>> EditableSkeletons;
};
