// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSkeletonTree.h"
#include "SkeletonTreeMenuContext.generated.h"

UCLASS()
class USkeletonTreeMenuContext : public UObject
{
	GENERATED_BODY()

public:
	/** The skeleton tree we reference */
	TWeakPtr<SSkeletonTree> SkeletonTree;
};