// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeSocketItem.h"
#include "Widgets/Text/STextBlock.h"
#include "SSkeletonTreeRow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SListView.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/Images/SImage.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "SocketDragDropOp.h"
#include "IPersonaPreviewScene.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreeSocketItem"

FSkeletonTreeSocketItem::FSkeletonTreeSocketItem(
	USkeletalMeshSocket* InSocket,
	ESocketParentType InParentType,
	bool bInIsCustomized,
	const TSharedRef<class ISkeletonTree>& InSkeletonTree
	) :
		FSkeletonTreeItem(InSkeletonTree),
		Socket(InSocket),
		ParentType(InParentType),
		bIsCustomized(bInIsCustomized),
		bInlineEditorExpanded(false)
{
	OnSocketChangedDelegate = InSocket->OnPropertyChanged().AddLambda(
		[SkeletonTree=InSkeletonTree.ToWeakPtr()](const USkeletalMeshSocket*, const FProperty* InProperty)
	{
		if (InProperty->GetName() == GET_MEMBER_NAME_CHECKED(USkeletalMeshSocket, BoneName))
		{
			if (TSharedPtr<ISkeletonTree> Tree = SkeletonTree.Pin())
			{
				Tree->Refresh();
			}
		}
	});
}

FSkeletonTreeSocketItem::~FSkeletonTreeSocketItem()
{
	Socket->OnPropertyChanged().Remove(OnSocketChangedDelegate);
}

void FSkeletonTreeSocketItem::GenerateWidgetForNameColumn( TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected )
{
	const FSlateBrush* SocketIcon = ( ParentType == ESocketParentType::Mesh ) ?
		FAppStyle::GetBrush( "SkeletonTree.MeshSocket" ) :
		FAppStyle::GetBrush( "SkeletonTree.SkeletonSocket" );

	Box->AddSlot()
	.AutoWidth()
	.Padding(FMargin(0.0f, 2.0f))
	[
		SNew( SImage )
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image( SocketIcon )
	];

	const FSlateFontInfo TextFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	FText ToolTip = FText::Format(LOCTEXT("SocketMainTreeItemTooltip", "{0}\n\nHold Alt to duplicate the socket.\nHold Shift to keep the same absolute location on the new parent bone."),
		GetSocketToolTip());

	TAttribute<FText> SocketNameAttr = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FSkeletonTreeSocketItem::GetSocketNameAsText));
	TSharedPtr< SInlineEditableTextBlock > InlineWidget;

	Box->AddSlot()
	.AutoWidth()
	.Padding(2, 0, 0, 0)
	[
		SAssignNew( InlineWidget, SInlineEditableTextBlock )
			.ColorAndOpacity( this, &FSkeletonTreeSocketItem::GetTextColor )
			.Text( SocketNameAttr )
			.HighlightText( FilterText )
			.Font( TextFont )
			.ToolTipText( ToolTip )
			.OnVerifyTextChanged( this, &FSkeletonTreeSocketItem::OnVerifySocketNameChanged )
			.OnTextCommitted( this, &FSkeletonTreeSocketItem::OnCommitSocketName )
			.IsSelected( InIsSelected )
	];

	OnRenameRequested.BindSP( InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	if ( ParentType == ESocketParentType::Mesh )
	{
		FText SocketSuffix = IsSocketCustomized() ?
			LOCTEXT( "CustomizedSuffix", " [Mesh]" ) :
			LOCTEXT( "MeshSuffix", " [Mesh Only]" );

		Box->AddSlot()
		.AutoWidth()
		[
			SNew( STextBlock )
			.ColorAndOpacity( FLinearColor::Gray )
			.Text( SocketSuffix )
			.Font(TextFont)
			.ToolTipText( ToolTip )
		];
	}
}

TSharedRef< SWidget > FSkeletonTreeSocketItem::GenerateInlineEditWidget(const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	if (GetDefault<UPersonaOptions>()->bUseInlineSocketEditor)
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bShowScrollBar = false;
		TSharedRef<IDetailsView> DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(Socket);

		return SNew(SOverlay)
			.Visibility_Lambda([this]() { return bInlineEditorExpanded ? EVisibility::Visible : EVisibility::Collapsed; })
			+ SOverlay::Slot()
			.Padding(2.0f, 4.0f, 2.0f, 4.0f)
			[
				DetailsView
			]
		+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
			.Image(FAppStyle::GetBrush("SkeletonTree.InlineEditorShadowTop"))
			]
		+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
			.Image(FAppStyle::GetBrush("SkeletonTree.InlineEditorShadowBottom"))
			];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

FSlateColor FSkeletonTreeSocketItem::GetTextColor() const
{
	if (ParentType == ESocketParentType::Skeleton && bIsCustomized)
	{
		return FSlateColor::UseSubduedForeground();
	}
	else
	{
		return FSlateColor::UseForeground();
	}
}

TSharedRef< SWidget > FSkeletonTreeSocketItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	return SNullWidget::NullWidget;
}

void FSkeletonTreeSocketItem::OnItemDoubleClicked()
{
	OnRenameRequested.ExecuteIfBound();
}

bool FSkeletonTreeSocketItem::CanCustomizeSocket() const
{
	// If the socket is on the skeleton, we have a valid mesh
	// and there isn't one of the same name on the mesh, we can customize it
	if (GetSkeletonTree()->GetPreviewScene().IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent();
		return PreviewComponent && PreviewComponent->GetSkeletalMeshAsset() && !IsSocketCustomized();
	}
	return false;
}

void FSkeletonTreeSocketItem::RequestRename()
{
	OnRenameRequested.ExecuteIfBound();
}

bool FSkeletonTreeSocketItem::OnVerifySocketNameChanged( const FText& InText, FText& OutErrorMessage )
{
	// You can't have two sockets with the same name on the mesh, nor on the skeleton,
	// but you can have a socket with the same name on the mesh *and* the skeleton.
	bool bVerifyName = true;

	FText NewText = FText::TrimPrecedingAndTrailing(InText);

	if (NewText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT( "EmptySocketName_Error", "Sockets must have a name!");
		bVerifyName = false;
	}
	else
	{
		USkeletalMesh* SkeletalMesh = GetSkeletonTree()->GetPreviewScene().IsValid() ? ToRawPtr(GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;
		bVerifyName = !GetEditableSkeleton()->DoesSocketAlreadyExist( Socket, NewText, ParentType, SkeletalMesh );

		// Needs to be checked on verify.
		if ( !bVerifyName )
		{

			// Tell the user that the socket is a duplicate
			OutErrorMessage = LOCTEXT( "DuplicateSocket_Error", "Socket name in use!");
		}
	}

	return bVerifyName;
}

void FSkeletonTreeSocketItem::OnCommitSocketName( const FText& InText, ETextCommit::Type CommitInfo )
{
	FText NewText = FText::TrimPrecedingAndTrailing(InText);

	// Notify skeleton tree of socket rename
	USkeletalMesh* SkeletalMesh = GetSkeletonTree()->GetPreviewScene().IsValid() ? ToRawPtr(GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()) : nullptr;
	GetEditableSkeleton()->RenameSocket(Socket->SocketName, FName(*NewText.ToString()), SkeletalMesh);
}

FReply FSkeletonTreeSocketItem::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		FSelectedSocketInfo SocketInfo( Socket, ParentType == ESocketParentType::Skeleton );

		return FReply::Handled().BeginDragDrop( FSocketDragDropOp::New( SocketInfo, MouseEvent.GetModifierKeys() ) );
	}

	return FReply::Unhandled();
}

FText FSkeletonTreeSocketItem::GetSocketToolTip() const
{
	FText ToolTip;

	if ( ParentType == ESocketParentType::Skeleton && bIsCustomized == false )
	{
		ToolTip = LOCTEXT( "SocketToolTipSkeletonOnly", "This socket is on the skeleton only. It is shared with all meshes that use this skeleton" );
	}
	else if ( ParentType == ESocketParentType::Mesh && bIsCustomized == false )
	{
		ToolTip = LOCTEXT( "SocketToolTipMeshOnly", "This socket is on the current mesh only" );
	}
	else if ( ParentType == ESocketParentType::Skeleton )
	{
		ToolTip = LOCTEXT( "SocketToolTipSkeleton", "This socket is on the skeleton (shared with all meshes that use the skeleton), and the current mesh has duplicated version of it" );
	}
	else
	{
		ToolTip = LOCTEXT( "SocketToolTipCustomized", "This socket is on the current mesh, customizing the socket of the same name on the skeleton" );
	}

	return ToolTip;
}

#undef LOCTEXT_NAMESPACE
