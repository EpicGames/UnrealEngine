// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "ISkeletonTreeItem.h"
#include "SkeletonTreeItem.h"
#include "IEditableSkeleton.h"
#include "Widgets/Input/SEditableText.h"
#include "Preferences/PersonaOptions.h"
#include "Engine/SkeletalMeshSocket.h"

class FSkeletonTreeSocketItem : public FSkeletonTreeItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreeSocketItem, FSkeletonTreeItem)

	FSkeletonTreeSocketItem(USkeletalMeshSocket* InSocket, ESocketParentType InParentType, bool bInIsCustomized, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	virtual ~FSkeletonTreeSocketItem() override;

	/** ISkeletonTreeItem interface */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateInlineEditWidget(const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual bool HasInlineEditor() const override { return GetDefault<UPersonaOptions>()->bUseInlineSocketEditor; }
	virtual void ToggleInlineEditorExpansion() override { bInlineEditorExpanded = !bInlineEditorExpanded; }
	virtual bool IsInlineEditorExpanded() const override { return bInlineEditorExpanded; }
	virtual FName GetRowItemName() const override { return Socket->SocketName; }
	virtual bool CanRenameItem() const override { return true; }
	virtual void RequestRename() override;
	virtual void OnItemDoubleClicked() override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual UObject* GetObject() const override { return Socket; }

	/** Get the parent type */
	ESocketParentType GetParentType() const { return ParentType; }

	/** Is this socket customized */
	bool IsSocketCustomized() const { return bIsCustomized; }

	/** Can we customize this socket? */
	bool CanCustomizeSocket() const;

	/** Get the socket we wrap */
	USkeletalMeshSocket* GetSocket() const { return Socket; }

private:
	/** Called when user is renaming a socket to verify the name **/
	bool OnVerifySocketNameChanged(const FText& InText, FText& OutErrorMessage);

	/** Called when user renames a socket **/
	void OnCommitSocketName(const FText& InText, ETextCommit::Type CommitInfo);

	/** Function that returns the current tooltip for this socket */
	FText GetSocketToolTip() const;

	/** Get the color to display for the socket name */
	FSlateColor GetTextColor() const;	

	/** Return socket name as FText for display in skeleton tree */
	FText GetSocketNameAsText() const { return FText::FromName(Socket->SocketName); }

private:
	/** The socket we represent */
	USkeletalMeshSocket* Socket;

	/** Whether we are on the skeleton or the mesh */
	ESocketParentType ParentType;

	/** Whether we are customized */
	bool bIsCustomized;

	/** Track expansion state of the in-line editor */
	bool bInlineEditorExpanded;

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

	FDelegateHandle OnSocketChangedDelegate;
};
