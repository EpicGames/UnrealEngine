// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeSummoner.h"
#include "Styling/AppStyle.h"
#include "IDocumentation.h"
#include "SkeletonEditor.h"

#define LOCTEXT_NAMESPACE "SkeletonTreeSummoner"

FSkeletonTreeSummoner::FSkeletonTreeSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, TSharedRef<ISkeletonTree> InSkeletonTree)
	: FWorkflowTabFactory(SkeletonEditorTabs::SkeletonTreeTab, InHostingApp)
	, SkeletonTreePtr(InSkeletonTree)
{
	TabLabel = LOCTEXT("SkeletonTreeTabTitle", "Skeleton Tree");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.SkeletonTree");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("SkeletonTreeView", "Skeleton Tree");
	ViewMenuTooltip = LOCTEXT("SkeletonTreeView_ToolTip", "Shows the skeleton tree");
}

TSharedPtr<SToolTip> FSkeletonTreeSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("SkeletonTreeTooltip", "The Skeleton Tree tab lets you see and select bones (and sockets) in the skeleton hierarchy."), NULL, TEXT("Shared/Editors/Persona"), TEXT("SkeletonTree_Window"));
}

TSharedRef<SWidget> FSkeletonTreeSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	if (SkeletonTreePtr.IsValid())
	{
		return SkeletonTreePtr.Pin().ToSharedRef();
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
