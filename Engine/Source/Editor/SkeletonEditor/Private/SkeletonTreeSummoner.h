// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ISkeletonTree.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class SToolTip;

struct FSkeletonTreeSummoner : public FWorkflowTabFactory
{
public:
	FSkeletonTreeSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, TSharedRef<class ISkeletonTree> InSkeletonTree);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our skeleton tree */
	TWeakPtr<class ISkeletonTree> SkeletonTreePtr;
};
