// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreeVirtualBoneItem.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"
#include "SSkeletonTreeRow.h"
#include "IPersonaPreviewScene.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Containers/UnrealString.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/BlendProfile.h"
#include "UObject/Package.h"
#include "SocketDragDropOp.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Preferences/PersonaOptions.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreeVirtualBoneItem"

FSkeletonTreeVirtualBoneItem::FSkeletonTreeVirtualBoneItem(const FName& InBoneName, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreeItem(InSkeletonTree)
	, BoneName(InBoneName)
{
	static const FString BoneProxyPrefix(TEXT("VIRTUALBONEPROXY_"));

	BoneProxy = NewObject<UBoneProxy>(GetTransientPackage(), *(BoneProxyPrefix + FString::Printf(TEXT("%p"), &InSkeletonTree.Get()) + InBoneName.ToString()));
	BoneProxy->SetFlags(RF_Transactional);
	BoneProxy->BoneName = InBoneName;
	BoneProxy->bIsTransformEditable = false;
	TSharedPtr<IPersonaPreviewScene> PreviewScene = InSkeletonTree->GetPreviewScene();
	if (PreviewScene.IsValid())
	{
		BoneProxy->SkelMeshComponent = PreviewScene->GetPreviewMeshComponent();
		BoneProxy->WeakPreviewScene = PreviewScene.ToWeakPtr();
	}
}

EVisibility FSkeletonTreeVirtualBoneItem::GetLODIconVisibility() const
{
	return EVisibility::Visible;
}

void FSkeletonTreeVirtualBoneItem::GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	const FSlateBrush* LODIcon = FAppStyle::Get().GetBrush("SkeletonTree.Bone");

	Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 2.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ColorAndOpacity(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextColor, InIsSelected)
			.Image(LODIcon)
			.Visibility(this, &FSkeletonTreeVirtualBoneItem::GetLODIconVisibility)
		];

	FText ToolTip = GetBoneToolTip();

	TAttribute<FText> NameAttr = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FSkeletonTreeVirtualBoneItem::GetVirtualBoneNameAsText));

	InlineWidget = SNew(SInlineEditableTextBlock)
						.ColorAndOpacity(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextColor, InIsSelected)
						.Text(NameAttr)
						.HighlightText(FilterText)
						.Font(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextFont)
						.ToolTipText(ToolTip)
						.OnEnterEditingMode(this, &FSkeletonTreeVirtualBoneItem::OnVirtualBoneNameEditing)
						.OnVerifyTextChanged(this, &FSkeletonTreeVirtualBoneItem::OnVerifyBoneNameChanged)
						.OnTextCommitted(this, &FSkeletonTreeVirtualBoneItem::OnCommitVirtualBoneName)
						.IsSelected(InIsSelected);

	OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	Box->AddSlot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		[
			SNew(STextBlock)
			.ColorAndOpacity(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextColor, InIsSelected)
			.Text(FText::FromString(VirtualBoneNameHelpers::VirtualBonePrefix))
			.Font(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextFont)
			.Visibility(this, &FSkeletonTreeVirtualBoneItem::GetVirtualBonePrefixVisibility)
		];

	Box->AddSlot()
		.Padding(4, 0, 0, 0)
		.AutoWidth()
		[
			InlineWidget.ToSharedRef()
		];
	
	Box->AddSlot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew (STextBlock)
			.ColorAndOpacity(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextColor, InIsSelected)
			.Text_Lambda( [BoneName=BoneName, EditableSkeleton=GetEditableSkeleton()]() -> FText
			{
				const int32 BoneIndex = EditableSkeleton->GetSkeleton().GetReferenceSkeleton().FindBoneIndex(BoneName);
				return FText::Format(LOCTEXT("VirtualBoneIndexValue", "<{0}>"), FText::AsNumber(BoneIndex));
			})
			.Font(this, &FSkeletonTreeVirtualBoneItem::GetBoneTextFont)
			.ToolTipText( LOCTEXT("VirtualBoneIndexTooltip", "The index of this virtual bone in the reference skeleton's flat list of bones.") )
			.Visibility_Lambda([]() -> EVisibility
			{
				return GetDefault<UPersonaOptions>()->bShowBoneIndexes ? EVisibility::Visible : EVisibility::Collapsed;
			})
		];
}

TSharedRef< SWidget > FSkeletonTreeVirtualBoneItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	if (DataColumnName == ISkeletonTree::Columns::BlendProfile)
	{
		return SNew(SBox)
			.Padding(0.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SSpinBox<float>)
				.Visibility(this, &FSkeletonTreeVirtualBoneItem::GetBoneBlendProfileVisibility)
			.Style(&FAppStyle::Get(), "SkeletonTree.HyperlinkSpinBox")
			.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
			.ContentPadding(0.0f)
			.Delta(0.01f)
			.MinValue(0.0f)
			.MinSliderValue(this, &FSkeletonTreeVirtualBoneItem::GetBlendProfileMinSliderValue)
			.MaxSliderValue(this, &FSkeletonTreeVirtualBoneItem::GetBlendProfileMaxSliderValue)
			.Value(this, &FSkeletonTreeVirtualBoneItem::GetBoneBlendProfileScale)
			.OnValueCommitted(this, &FSkeletonTreeVirtualBoneItem::OnBlendSliderCommitted)
			.OnValueChanged(this, &FSkeletonTreeVirtualBoneItem::OnBlendSliderChanged)
			.OnBeginSliderMovement(this, &FSkeletonTreeVirtualBoneItem::OnBeginBlendSliderMovement)
			.OnEndSliderMovement(this, &FSkeletonTreeVirtualBoneItem::OnEndBlendSliderMovement)
			.ClearKeyboardFocusOnCommit(true)
			];
	}

	return SNullWidget::NullWidget;
}

EVisibility FSkeletonTreeVirtualBoneItem::GetBoneBlendProfileVisibility() const
{
	return GetSkeletonTree()->GetSelectedBlendProfile() ? EVisibility::Visible : EVisibility::Collapsed;
}

float FSkeletonTreeVirtualBoneItem::GetBoneBlendProfileScale()	const
{
	if (UBlendProfile* CurrentProfile = GetSkeletonTree()->GetSelectedBlendProfile())
	{
		return CurrentProfile->GetBoneBlendScale(BoneName);
	}

	return 0.0;
}

TOptional<float> FSkeletonTreeVirtualBoneItem::GetBlendProfileMaxSliderValue() const
{
	if (UBlendProfile* CurrentProfile = GetSkeletonTree()->GetSelectedBlendProfile())
	{
		return (CurrentProfile->GetMode() == EBlendProfileMode::WeightFactor) ? 10.0f : 1.0f;
	}

	return 1.0f;
}

TOptional<float> FSkeletonTreeVirtualBoneItem::GetBlendProfileMinSliderValue() const
{
	if (UBlendProfile* CurrentProfile = GetSkeletonTree()->GetSelectedBlendProfile())
	{
		return (CurrentProfile->GetMode() == EBlendProfileMode::WeightFactor) ? 1.0f : 0.0f;
	}

	return 0.0f;
}

void FSkeletonTreeVirtualBoneItem::OnBeginBlendSliderMovement()
{
	if (bBlendSliderStartedTransaction == false)
	{
		bBlendSliderStartedTransaction = true;
		GEditor->BeginTransaction(LOCTEXT("BlendSliderTransation", "Set Blend Profile Value"));

		const FName& BlendProfileName = GetSkeletonTree()->GetSelectedBlendProfile()->GetFName();
		UBlendProfile* BlendProfile = GetEditableSkeleton()->GetBlendProfile(BlendProfileName);

		if (BlendProfile)
		{
			BlendProfile->SetFlags(RF_Transactional);
			BlendProfile->Modify();
		}
	}
}

void FSkeletonTreeVirtualBoneItem::OnEndBlendSliderMovement(float NewValue)
{
	if (bBlendSliderStartedTransaction)
	{
		GEditor->EndTransaction();
		bBlendSliderStartedTransaction = false;
	}
}

void FSkeletonTreeVirtualBoneItem::OnBlendSliderCommitted(float NewValue, ETextCommit::Type CommitType)
{
	FName BlendProfileName = GetSkeletonTree()->GetSelectedBlendProfile()->GetFName();
	UBlendProfile* BlendProfile = GetEditableSkeleton()->GetBlendProfile(BlendProfileName);

	if (BlendProfile)
	{
		FScopedTransaction Transaction(LOCTEXT("SetBlendProfileValue", "Set Blend Profile Value"));
		BlendProfile->SetFlags(RF_Transactional);
		BlendProfile->Modify();

		BlendProfile->SetBoneBlendScale(BoneName, NewValue, false, true);
	}
}

void FSkeletonTreeVirtualBoneItem::OnBlendSliderChanged(float NewValue)
{
	const FName& BlendProfileName = GetSkeletonTree()->GetSelectedBlendProfile()->GetFName();
	UBlendProfile* BlendProfile = GetEditableSkeleton()->GetBlendProfile(BlendProfileName);

	if (BlendProfile)
	{
		BlendProfile->SetBoneBlendScale(BoneName, NewValue, false, true);
	}
}

FSlateFontInfo FSkeletonTreeVirtualBoneItem::GetBoneTextFont() const
{
	return FAppStyle::GetWidgetStyle<FTextBlockStyle>("SkeletonTree.NormalFont").Font;
}

FSlateColor FSkeletonTreeVirtualBoneItem::GetBoneTextColor(FIsSelected InIsSelected) const
{
	bool bIsSelected = false;
	if (InIsSelected.IsBound())
	{
		bIsSelected = InIsSelected.Execute();
	}

	if(bIsSelected)
	{
		return FSlateColor::UseForeground();
	}
	else
	{
		return FSlateColor(FLinearColor(0.4f, 0.4f, 1.f));
	}
}

FText FSkeletonTreeVirtualBoneItem::GetBoneToolTip()
{
	return LOCTEXT("VirtualBone_ToolTip", "Virtual Bones are added in editor and allow space switching between two different bones in the skeleton.");
}

void FSkeletonTreeVirtualBoneItem::OnItemDoubleClicked()
{
	OnRenameRequested.ExecuteIfBound();
}

void FSkeletonTreeVirtualBoneItem::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSocketDragDropOp> DragConnectionOp = DragDropEvent.GetOperationAs<FSocketDragDropOp>();

	// Is someone dragging a socket onto a bone?
	if (DragConnectionOp.IsValid())
	{
		if (BoneName != DragConnectionOp->GetSocketInfo().Socket->BoneName)
		{
			// The socket can be dropped here if we're a bone and NOT the socket's existing parent
			DragConnectionOp->SetIcon(FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Ok")));
		}
		else if (DragConnectionOp->IsAltDrag())
		{
			// For Alt-Drag, dropping onto the existing parent is fine, as we're going to copy, not move the socket
			DragConnectionOp->SetIcon(FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Ok")));
		}
	}
}

void FSkeletonTreeVirtualBoneItem::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSocketDragDropOp> DragConnectionOp = DragDropEvent.GetOperationAs<FSocketDragDropOp>();
	if (DragConnectionOp.IsValid())
	{
		// Reset the drag/drop icon when leaving this row
		DragConnectionOp->SetIcon(FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
	}
}

FReply FSkeletonTreeVirtualBoneItem::HandleDrop(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSocketDragDropOp> DragConnectionOp = DragDropEvent.GetOperationAs<FSocketDragDropOp>();
	if (DragConnectionOp.IsValid())
	{
		FSelectedSocketInfo SocketInfo = DragConnectionOp->GetSocketInfo();

		if (DragConnectionOp->IsAltDrag())
		{
			// In an alt-drag, the socket can be dropped on any bone
			// (including its existing parent) to create a uniquely named copy
			GetSkeletonTree()->DuplicateAndSelectSocket(SocketInfo, BoneName);
		}
		else if (BoneName != SocketInfo.Socket->BoneName)
		{
			// The socket can be dropped here if we're a bone and NOT the socket's existing parent
			USkeletalMesh* SkeletalMesh = GetSkeletonTree()->GetPreviewScene().IsValid() ? GetSkeletonTree()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset() : nullptr;
			GetEditableSkeleton()->SetSocketParent(SocketInfo.Socket->SocketName, BoneName, SkeletalMesh);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void FSkeletonTreeVirtualBoneItem::RequestRename()
{
	OnRenameRequested.ExecuteIfBound();
}

void FSkeletonTreeVirtualBoneItem::OnVirtualBoneNameEditing()
{
	CachedBoneNameForRename = BoneName;
	BoneName = VirtualBoneNameHelpers::RemoveVirtualBonePrefix(BoneName.ToString());
}

bool FSkeletonTreeVirtualBoneItem::OnVerifyBoneNameChanged(const FText& InText, FText& OutErrorMessage)
{
	bool bVerifyName = true;

	FString InTextTrimmed = FText::TrimPrecedingAndTrailing(InText).ToString();

	FString NewName = VirtualBoneNameHelpers::AddVirtualBonePrefix(InTextTrimmed);

	if (InTextTrimmed.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyVirtualBoneName_Error", "Virtual bones must have a name!");
		bVerifyName = false;
	}
	else
	{
		if(InTextTrimmed != BoneName.ToString())
		{
			bVerifyName = !GetEditableSkeleton()->DoesVirtualBoneAlreadyExist(NewName);

			// Needs to be checked on verify.
			if (!bVerifyName)
			{

				// Tell the user that the name is a duplicate
				OutErrorMessage = LOCTEXT("DuplicateVirtualBone_Error", "Name in use!");
				bVerifyName = false;
			}
		}
	}

	return bVerifyName;
}

void FSkeletonTreeVirtualBoneItem::OnCommitVirtualBoneName(const FText& InText, ETextCommit::Type CommitInfo)
{
	FString NewNameString = VirtualBoneNameHelpers::AddVirtualBonePrefix(FText::TrimPrecedingAndTrailing(InText).ToString());
	FName NewName(*NewNameString);

	// Notify skeleton tree of rename
	GetEditableSkeleton()->RenameVirtualBone(CachedBoneNameForRename, NewName);
	BoneName = NewName;
}

EVisibility FSkeletonTreeVirtualBoneItem::GetVirtualBonePrefixVisibility() const
{
	return InlineWidget->IsInEditMode() ? EVisibility::Visible : EVisibility::Collapsed;
}

void FSkeletonTreeVirtualBoneItem::EnableBoneProxyTick(bool bEnable)
{
	BoneProxy->bIsTickable = bEnable;
}

void FSkeletonTreeVirtualBoneItem::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(BoneProxy);
}

#undef LOCTEXT_NAMESPACE
