// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "ISkeletonTreeItem.h"
#include "SkeletonTreeItem.h"
#include "IEditableSkeleton.h"
#include "BoneProxy.h"
#include "UObject/GCObject.h"

class SInlineEditableTextBlock;

class FSkeletonTreeVirtualBoneItem : public FSkeletonTreeItem, public FGCObject
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreeVirtualBoneItem, FSkeletonTreeItem)

	FSkeletonTreeVirtualBoneItem(const FName& InBoneName, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	/** Builds the table row widget to display this info */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override;
	virtual FName GetRowItemName() const override { return BoneName; }
	virtual bool CanRenameItem() const override { return true; }
	virtual void RequestRename() override;
	virtual void OnItemDoubleClicked() override;
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) override;
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) override;
	virtual UObject* GetObject() const override { return BoneProxy; }

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSkeletonTreeVirtualBoneItem");
	}

	/** Return socket name as FText for display in skeleton tree */
	FText GetVirtualBoneNameAsText() const { return FText::FromName(BoneName); }

	/** Enable and disable the bone proxy ticking */
	void EnableBoneProxyTick(bool bEnable);

private:
	
	/** Callback from a slider widget if the text entry is used */
	void OnBlendSliderCommitted(float NewValue, ETextCommit::Type CommitType);

	/** Callback from a slider widget if the slider is used */
	void OnBlendSliderChanged(float NewValue);

	/** Callback from a slider widget when the user begins sliding */
	void OnBeginBlendSliderMovement();

	/** Callback from a slider widget when the user has finished sliding */
	void OnEndBlendSliderMovement(float NewValue);

	/** Set Translation Retargeting Mode for this bone. */
	void SetBoneTranslationRetargetingMode(EBoneTranslationRetargetingMode::Type NewRetargetingMode);

	/** Get the current Blend Scale for this bone */
	float GetBoneBlendProfileScale() const;

	/** Get the max slider value supported by the selected blend profile's mode*/
	TOptional<float> GetBlendProfileMaxSliderValue() const;

	/** Get the min slider value supported by the selected blend profile's mode*/
	TOptional<float> GetBlendProfileMinSliderValue() const;

	/** Used to hide the Bone Profile row widget if there is no current Blend Profile */
	EVisibility GetBoneBlendProfileVisibility() const;
	
	/** Called when we are about to rename a virtual bone */
	void OnVirtualBoneNameEditing();

	/** Called when user is renaming this bone to verify the name **/
	bool OnVerifyBoneNameChanged(const FText& InText, FText& OutErrorMessage);

	/** Called when user renames this bone **/
	void OnCommitVirtualBoneName(const FText& InText, ETextCommit::Type CommitInfo);

	/** Called to get the visibility of the Prefix label during rename*/
	EVisibility GetVirtualBonePrefixVisibility() const;

	/** Gets the font for displaying bone text in the skeletal tree */
	FSlateFontInfo GetBoneTextFont() const;

	/** Get the text color based on bone part of skeleton or part of mesh */
	FSlateColor GetBoneTextColor(FIsSelected InIsSelected) const;

	/** visibility of the icon */
	EVisibility GetLODIconVisibility() const;

	/** Function that returns the current tooltip for this bone, depending on how it's used by the mesh */
	FText GetBoneToolTip();

	/** The actual bone data that we create Slate widgets to display */
	FName BoneName;

	/** During rename we modify the name slightly (strip off the VB prefix) cache the original name here*/
	FName CachedBoneNameForRename;

	/** Box for the user to type in the name - stored here so that SSkeletonTree can set the keyboard focus in Slate */
	//TSharedPtr<SEditableText> NameEntryBox;
	TSharedPtr< SInlineEditableTextBlock > InlineWidget;

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

	/** Bone proxy used for debug display */
	TObjectPtr<UBoneProxy> BoneProxy;

	/** True if the user is in a transaction when moving the blend profile slider */
	bool bBlendSliderStartedTransaction;
};
