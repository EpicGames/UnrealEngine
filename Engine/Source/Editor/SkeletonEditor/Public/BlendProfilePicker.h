// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UBlendProfile;

DECLARE_DELEGATE_OneParam(FOnBlendProfileSelected, UBlendProfile*);

enum class EBlendProfilePickerMode : uint8
{
	BlendProfile = (1U << 0),
	BlendMask = (1U << 1),
	AllModes = BlendProfile | BlendMask,
};
ENUM_CLASS_FLAGS(EBlendProfilePickerMode);

/** Argument used to set up the blend profile picker */
struct FBlendProfilePickerArgs
{
	/** Initial blend profile selected */
	UBlendProfile* InitialProfile;

	/** Delegate to call when the picker selection is changed */
	FOnBlendProfileSelected  OnBlendProfileSelected;

	/** Allow the option to create new profiles in the picker */
	bool bAllowNew;

	/** Allow the option to clear the profile selection */
	bool bAllowClear;

	/** Allow the option to delete blend profiles from the skeleton */
	bool bAllowModify;

	/** Only display Blend Profiles w/ specified Blend Profile modes (EBlendProfilePickerMode values are flags.) */
	EBlendProfilePickerMode SupportedBlendProfileModes;

	/** Optional property handle, when the picker is tied to a property */
	TSharedPtr<class IPropertyHandle> PropertyHandle;

	FBlendProfilePickerArgs()
		: InitialProfile(nullptr)
		, OnBlendProfileSelected()
		, bAllowNew(false)
		, bAllowClear(false)
		, bAllowModify(false)
		, SupportedBlendProfileModes(EBlendProfilePickerMode::AllModes)
		, PropertyHandle(nullptr)
	{}
};
