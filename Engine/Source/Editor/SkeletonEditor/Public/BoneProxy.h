// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "TickableEditorObject.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "IPersonaPreviewScene.h"
#include "BoneProxy.generated.h"

#define UE_API SKELETONEDITOR_API

class UDebugSkelMeshComponent;
enum class ETransformType : uint8;
namespace ESlateTransformComponent { enum Type : int; }
namespace ESlateRotationRepresentation { enum Type : int; }
namespace ESlateTransformSubComponent { enum Type : int; }
namespace ETextCommit { enum Type : int; }

/** Proxy object used to display/edit bone transforms */
UCLASS(MinimalAPI)
class UBoneProxy : public UObject, public FTickableEditorObject
{
	GENERATED_BODY()

public:
	UE_API UBoneProxy();

	enum ETransformType
	{
		TransformType_Bone,
		TransformType_Reference,
		TransformType_Mesh
	};

	enum class ESliderMovementState
	{
		Begin,
		End
	};

	
	/** UObject interface */
	UE_API virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// overload based on a property name
	UE_API void OnPreEditChange(FName PropertyName);
	UE_API void OnPostEditChangeProperty(FName PropertyName);
	
	/** FTickableEditorObject interface */
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual bool IsTickable() const override;
	UE_API virtual TStatId GetStatId() const override;

	/** The name of the bone we have selected */
	UPROPERTY(VisibleAnywhere, Category = "Bone")
	FName BoneName;

	/** Bone location */
	UPROPERTY(EditAnywhere, Category = "Transform")
	FVector Location;

	/** Bone rotation */
	UPROPERTY(EditAnywhere, Category = "Transform")
	FRotator Rotation;

	/** Bone scale */
	UPROPERTY(EditAnywhere, Category = "Transform")
	FVector Scale;

	/** Bone reference location (local) */
	UPROPERTY(VisibleAnywhere, Category = "Reference Transform")
	FVector ReferenceLocation;

	/** Bone reference rotation (local) */
	UPROPERTY(VisibleAnywhere, Category = "Reference Transform")
	FRotator ReferenceRotation;

	/** Bone reference scale (local) */
	UPROPERTY(VisibleAnywhere, Category = "Reference Transform")
	FVector ReferenceScale;

	/** Mesh Location. Non zero when processing root motion */
	UPROPERTY(VisibleAnywhere, Category = "Mesh Relative Transform")
	FVector MeshLocation;

	/** Mesh Rotation. Non zero when processing root motion */
	UPROPERTY(VisibleAnywhere, Category = "Mesh Relative Transform")
	FRotator MeshRotation;

	/** Mesh Scale */
	UPROPERTY(VisibleAnywhere, Category = "Mesh Relative Transform")
	FVector MeshScale;

	/** The skeletal mesh component we glean our transform data from */
	UPROPERTY()
	TWeakObjectPtr<UDebugSkelMeshComponent> SkelMeshComponent;

	TWeakPtr<IPersonaPreviewScene> WeakPreviewScene;

	/** Whether to use local or world location */
	bool bLocalLocation;

	/** Whether to use local or world rotation */
	bool bLocalRotation;

	/** Whether to use local or world rotation */
	bool bLocalScale;

	// Handle property deltas
	FVector PreviousLocation;
	FRotator PreviousRotation;
	FVector PreviousScale;

	/** Flag indicating we are in the middle of a drag operation */
	bool bManipulating;

	/** Flag indicating whether this FTickableEditorObject should actually tick */
	bool bIsTickable;

	/** Flag indicating whether this bone's transform is editable */
	bool bIsTransformEditable;

	/** Method to react to retrieval of numeric values for the widget */
	UE_API TOptional<FVector::FReal> GetNumericValue(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		ETransformType TransformType) const;

	/** Static method to retrieve a value for a list of proxies */
	static UE_API TOptional<FVector::FReal> GetMultiNumericValue(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		ETransformType TransformType,
		TArrayView<UBoneProxy*> BoneProxies);

	/** Method to react to changes of numeric values in the widget */
	UE_API void OnNumericValueChanged(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		ETransformType TransformType);

	static UE_API void OnSliderMovementStateChanged(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		ESliderMovementState SliderMovementState,
		TArrayView<UBoneProxy*> BoneProxies);

	/** Method to react to changes of numeric values in the widget */
	static UE_API void OnMultiNumericValueChanged(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		ETextCommit::Type InCommitType,
		bool bInTransactional,
		ETransformType TransformType,
		TArrayView<UBoneProxy*> BoneProxies);

	// Returns true if a given transform component differs from the reference
	UE_API bool DiffersFromDefault(ESlateTransformComponent::Type Component, ETransformType TransformType) const;

	// Resets the bone transform's component to its reference
	UE_API void ResetToDefault(ESlateTransformComponent::Type InComponent, ETransformType TransformType);

private:
	static UE_API void BeginSetValueTransaction(ESlateTransformComponent::Type InComponent, TArrayView<UBoneProxy*> BoneProxies);
	static UE_API void EndTransaction();
};

#undef UE_API
