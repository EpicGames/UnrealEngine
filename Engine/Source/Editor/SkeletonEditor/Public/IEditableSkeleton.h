// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneContainer.h"
#include "Animation/Skeleton.h"

enum class EBlendProfileMode : uint8;
class UBlendProfile;
class USkeletalMesh;

/** Delegate fired when a set of smart names is removed */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSmartNameChanged, const FName& /*InContainerName*/);

/** Enum which tells us whether the parent of a socket is the skeleton or skeletal mesh */
enum class ESocketParentType : int32
{
	Skeleton,
	Mesh
};

/** Interface used to deal with skeletons that are currently being edited */
class IEditableSkeleton
{
public:
	/** Get the skeleton. Const - to modify the skeleton you should use the wrapper methods below. Use IsSkeletonValid first, as the Skeleton is not guaranteed to exist. */
	virtual const class USkeleton& GetSkeleton() const = 0;

	/** Checks to see if the editable skeleton is valid. It is possible for Skeleton to be non-existent (if deleted externally). */
	virtual bool IsSkeletonValid() const = 0;
	
	/** Get the blend profiles that this skeleton currently contains */
	virtual const TArray<class UBlendProfile*>& GetBlendProfiles() const  = 0;

	/** Get the named blend profile @return nullptr if none was found */
	virtual class UBlendProfile* GetBlendProfile(const FName& InBlendProfileName) = 0;

	/** Create the named blend profile */
	virtual class UBlendProfile* CreateNewBlendProfile(const FName& InBlendProfileName) = 0;

	/** Remove the specifed blend profile */
	virtual void RemoveBlendProfile(UBlendProfile* InBlendProfile) = 0;

	/** Rename an existing blend profile */
	virtual class UBlendProfile* RenameBlendProfile(const FName& InBlendProfileName, const FName& InNewBlendProfileName) = 0;

	/** Set the blend profile scale for the specified bone */
	virtual void SetBlendProfileScale(const FName& InBlendProfileName, const FName& InBoneName, float InNewScale, bool bInRecurse) = 0;

	/** Change an existing blend profile's mode (See EBlendProfileMode for details) */
	virtual void SetBlendProfileMode(FName InBlendProfileName, EBlendProfileMode ProfileMode) = 0;

	/** Creates a new socket on the skeleton */
	virtual USkeletalMeshSocket* AddSocket(const FName& InBoneName) = 0;

	/** Handle drag-drop socket duplication */
	virtual class USkeletalMeshSocket* DuplicateSocket(const struct FSelectedSocketInfo& SocketInfoToDuplicate, const FName& NewParentBoneName, USkeletalMesh* InSkeletalMesh) = 0;

	/** Rename a socket on the mesh or the skeleton */
	virtual void RenameSocket(const FName OldSocketName, const FName NewSocketName, USkeletalMesh* InSkeletalMesh) = 0;

	/** Set the parent of a socket */
	virtual void SetSocketParent(const FName& SocketName, const FName& NewParentName, USkeletalMesh* InSkeletalMesh, bool bKeepAbsoluteLocation = false) = 0;

	/** Function to tell you if a socket name already exists. SocketParentType determines where we will look to see if the socket exists, i.e. mesh or skeleton */
	virtual bool DoesSocketAlreadyExist(const class USkeletalMeshSocket* InSocket, const FText& InSocketName, ESocketParentType SocketParentType, USkeletalMesh* InSkeletalMesh) const = 0;

	/**
	 * Makes sure all attached objects are valid and removes any that aren't.
	 *
	 * @return		NumberOfBrokenAssets
	 */
	virtual int32 ValidatePreviewAttachedObjects() = 0;

	/** 
	 * Merge has failed, then Recreate BoneTree
	 * 
	 * This will invalidate all animations that were linked before, but this is needed 
	 * 
	 * @param InSkelMesh			: Mesh to build from. 
	 * 
	 * @return true if success
	 */	
	virtual void RecreateBoneTree(class USkeletalMesh* NewPreviewMesh) = 0;

	/** 
	 * Delete anim notifies by name
	 * @return the number of animations modified
	 */	
	virtual int32 DeleteAnimNotifies(const TArray<FName>& InotifyNames, bool bDeleteFromAnimations = true) = 0;

	/**
	 * Delete sync markers from the skeleton cache by name
	 * @return the number of animations modified
	 */
	virtual int32 DeleteSyncMarkers(const TArray<FName>& ISyncMarkerNames, bool bDeleteFromAnimations = true) = 0;

	/**
	* Add a notify
	*/
	virtual void AddNotify(FName NewName) = 0;

	/** 
	 * Rename a notify
	 * @return the number of animations modified
	 */	
	virtual int32 RenameNotify(const FName NewName, const FName OldName, bool bRenameInAnimations = true) = 0;

	/**
	* Add a sync marker
	*/
	virtual void AddSyncMarker(FName NewName) = 0;

	/**
	* Rename a sync marker
	 * @return the number of animations modified
	*/
	virtual int32 RenameSyncMarker(FName NewName, const FName OldName, bool bRenameInAnimations = true) = 0;
	
	/** Inform the system that something about a notify changed */	
	virtual void BroadcastNotifyChanged() = 0;

	/** Populates OutAssets with the AnimSequences that match this current skeleton */
	virtual void GetCompatibleAnimSequences(TArray<struct FAssetData>& OutAssets) = 0;

	/** Set the preview mesh in the skeleton*/
	virtual void SetPreviewMesh(class USkeletalMesh* InSkeletalMesh) = 0;

	/** Load any additional meshes we may have */
	virtual void LoadAdditionalPreviewSkeletalMeshes() = 0;

	/** Set the additional skeletal meshes we use when previewing this skeleton */
	virtual void SetAdditionalPreviewSkeletalMeshes(class UDataAsset* InPreviewCollectionAsset) = 0;

	/** Rename the specified retarget source */
	virtual void RenameRetargetSource(const FName InOldName, const FName InNewName) = 0;

	/** Add a retarget source */
	virtual void AddRetargetSource(const FName& InName, USkeletalMesh* InReferenceMesh) = 0;

	/** Delete retarget sources */
	virtual void DeleteRetargetSources(const TArray<FName>& InRetargetSourceNames) = 0;

	/** Refresh retarget sources */
	virtual void RefreshRetargetSources(const TArray<FName>& InRetargetSourceNames) = 0;

	/** Add a compatible skeleton */
	virtual void AddCompatibleSkeleton(const USkeleton* InCompatibleSkeleton) = 0;

	/** Remove a compatible skeleton */
	virtual void RemoveCompatibleSkeleton(const USkeleton* InCompatibleSkeleton) = 0;

	/** Remove any bones that are not used by any skeletal meshes */
	virtual void RemoveUnusedBones() = 0;

	/** Returns a list of all bones that are not used by any skeletal meshes */
	virtual void FindUnusedBones(TArray<FName>& OutUnusedBones, bool bSilent = false) = 0;

	/** Create RefLocalPoses from InSkeletalMesh */
	virtual void UpdateSkeletonReferencePose(class USkeletalMesh* InSkeletalMesh) = 0;

	/** Register a slot node */
	virtual void RegisterSlotNode(const FName& InSlotName) = 0;

	/** Add a slot group name */
	virtual bool AddSlotGroupName(const FName& InSlotName) = 0;

	/** Set the group name of the specified slot */
	virtual void SetSlotGroupName(const FName& InSlotName, const FName& InGroupName) = 0;

	/** Delete a slot name */
	virtual void DeleteSlotName(const FName& InSlotName) = 0;

	/** Delete a slot group */
	virtual void DeleteSlotGroup(const FName& InGroupName) = 0;

	/** Rename a slot name */
	virtual void RenameSlotName(const FName InOldSlotName, const FName InNewSlotName) = 0;

	/** Register a delegate to be called when this skeletons notifies are changed */
	virtual void RegisterOnNotifiesChanged(const FSimpleMulticastDelegate::FDelegate& InDelegate) = 0;

	/** Unregister a delegate to be called when this skeletons notifies are changed */
	virtual void UnregisterOnNotifiesChanged(FDelegateUserObject Thing) = 0;

	/** Register a delegate to be called when this skeletons slots are changed */
	virtual FDelegateHandle RegisterOnSlotsChanged(const FSimpleMulticastDelegate::FDelegate& InDelegate) = 0;

	/** Unregister a delegate to be called when this skeletons slots are changed */
	virtual void UnregisterOnSlotsChanged(FDelegateHandle InHandle) = 0;

	/** Wrap USkeleton::SetBoneTranslationRetargetingMode */
	virtual void SetBoneTranslationRetargetingMode(FName InBoneName, EBoneTranslationRetargetingMode::Type NewRetargetingMode) = 0;

	/** Wrap USkeleton::GetBoneTranslationRetargetingMode */
	virtual EBoneTranslationRetargetingMode::Type GetBoneTranslationRetargetingMode(FName InBoneName) const = 0;

	/** Function to tell you if a virtual bone name is already in use */
	virtual bool DoesVirtualBoneAlreadyExist(const FString& InVBName) const = 0;

	/** Rename an existing virtual bone */
	virtual void RenameVirtualBone(const FName OriginalName, const FName InVBName) = 0;

	/** Broadcasts tree refresh delegate */
	virtual void RefreshBoneTree() = 0;
};
