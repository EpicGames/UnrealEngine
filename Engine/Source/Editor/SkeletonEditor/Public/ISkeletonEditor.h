// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PersonaAssetEditorToolkit.h"
#include "IHasPersonaToolkit.h"

class IAnimationSequenceBrowser;

class ISkeletonEditor : public FPersonaAssetEditorToolkit, public IHasPersonaToolkit
{
	public:
	/** Get the asset browser we host */
	virtual IAnimationSequenceBrowser* GetAssetBrowser() const = 0;
};
