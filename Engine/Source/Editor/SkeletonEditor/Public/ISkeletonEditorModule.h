// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ISkeletonEditor.h"
#include "ISkeletonTree.h"
#include "BlendProfilePicker.h"

class IEditableSkeleton;
class USkeleton;

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletonEditor, Log, All);

class ISkeletonEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Creates a new skeleton editor instance */
	virtual TSharedRef<ISkeletonEditor> CreateSkeletonEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, class USkeleton* InSkeleton) = 0;

	/** Creates a new skeleton tree instance */
	virtual TSharedRef<ISkeletonTree> CreateSkeletonTree(USkeleton* InSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs) = 0;

	/** Creates a new skeleton tree instance */
	virtual TSharedRef<ISkeletonTree> CreateSkeletonTree(const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const FSkeletonTreeArgs& InSkeletonTreeArgs) = 0;

	/** Creates a new skeleton tree instance & registers a tab factory with the supplied tab factories */
	virtual TSharedRef<class FWorkflowTabFactory> CreateSkeletonTreeTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<ISkeletonTree>& InSkeletonTree) = 0;

	/** Creates a new editable skeleton instance */
	virtual TSharedRef<IEditableSkeleton> CreateEditableSkeleton(USkeleton* InSkeleton) = 0;

	/** Creates a new blend profile picker instance */
	virtual TSharedRef<SWidget> CreateBlendProfilePicker(USkeleton* InSkeleton, const FBlendProfilePickerArgs& InArgs = FBlendProfilePickerArgs()) = 0;

	/** Get all toolbar extenders */
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FSkeletonEditorToolbarExtender, const TSharedRef<FUICommandList> /*InCommandList*/, TSharedRef<ISkeletonEditor> /*InSkeletonEditor*/);
	virtual TArray<FSkeletonEditorToolbarExtender>& GetAllSkeletonEditorToolbarExtenders() = 0;
};
