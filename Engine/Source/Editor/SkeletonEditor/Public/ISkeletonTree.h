// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Containers/ArrayView.h"

struct FAssetData;
class IPersonaPreviewScene;
class UBlendProfile;
struct FSelectedSocketInfo;
class ISkeletonTreeBuilder;
class ISkeletonTreeItem;
class FMenuBuilder;
class FExtender;
class IPinnedCommandList;

// Called when an item is selected/deselected
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSkeletonTreeSelectionChangedMulticast, const TArrayView<TSharedPtr<ISkeletonTreeItem>>& /* InSelectedItems */, ESelectInfo::Type /* SelectInfo */);

// Called when an item is selected/deselected
typedef FOnSkeletonTreeSelectionChangedMulticast::FDelegate FOnSkeletonTreeSelectionChanged;

// Called when a bone is selected - DEPRECATED, please use FOnSkeletonTreeSelectionChangedMulticast
DECLARE_MULTICAST_DELEGATE_OneParam(FOnObjectSelectedMulticast, UObject* /* InObject */);

// Called when an object is selected - DEPRECATED, please use FOnSkeletonTreeSelectionChanged
typedef FOnObjectSelectedMulticast::FDelegate FOnObjectSelected;

/** Delegate that allows custom filtering text to be shown on the filter button */
DECLARE_DELEGATE_OneParam(FOnGetFilterText, TArray<FText>& /*InOutTextItems*/);

enum class ESkeletonTreeMode
{
	/** Skeleton tree allows editing */
	Editor,

	/** Skeleton tree allows picking of tree elements */
	Picker,
};

/** Init params for a skeleton tree widget */
struct FSkeletonTreeArgs
{
	FSkeletonTreeArgs()
		: Mode(ESkeletonTreeMode::Editor)
		, ContextName(TEXT("SkeletonTree"))
		, bShowBlendProfiles(true)
		, bShowFilterMenu(true)
		, bShowDebugVisualizationOptions(false)
		, bAllowMeshOperations(true)
		, bAllowSkeletonOperations(true)
		, bHideBonesByDefault(false)
		, bAllowInvisibleItemSelection(false)
	{}

	/** Delegate called by the tree when a socket is selected */
	FOnSkeletonTreeSelectionChanged OnSelectionChanged;

	/** Delegate that allows custom filtering text to be shown on the filter button */
	FOnGetFilterText OnGetFilterText;

	/** Optional preview scene that we can pair with */
	TSharedPtr<IPersonaPreviewScene> PreviewScene;

	/** Optional builder to allow for custom tree construction */
	TSharedPtr<ISkeletonTreeBuilder> Builder;

	/** Menu extenders applied to context and filter menus */
	TSharedPtr<FExtender> Extenders;

	/** The mode that this skeleton tree is in */
	ESkeletonTreeMode Mode;

	/** Context name used to persist settings */
	FName ContextName;

	/** Whether to show the blend profiles editor for the skeleton being displayed */
	bool bShowBlendProfiles;

	/** Whether to show the filter menu to allow filtering of active bones, sockets etc. */
	bool bShowFilterMenu;

	/** Whether to show the filter option to allow filtering of debug draw elements in the viewport. */
	bool bShowDebugVisualizationOptions;

	/** Whether to allow operations that modify the mesh */
	bool bAllowMeshOperations;

	/** Whether to allow operations that modify the skeleton */
	bool bAllowSkeletonOperations;

	/** Whether to hide bones by default */
	bool bHideBonesByDefault;

	/** Wether children can remain selected while their parent is collapsed in the tree */
	bool bAllowInvisibleItemSelection;
};

/** Interface used to deal with skeleton editing UI */
class ISkeletonTree : public SCompoundWidget
{
public:
	struct Columns
	{
		static SKELETONEDITOR_API const FName Name;
		static SKELETONEDITOR_API const FName Retargeting;
		static SKELETONEDITOR_API const FName BlendProfile;
		static SKELETONEDITOR_API const FName DebugVisualization;
	};

	static const FName DebugVisualizationOptionsName() { return Columns::DebugVisualization; };

	/** Manually refresh the tree */
	virtual void Refresh() = 0;

	/** Manually refresh the tree filter */
	virtual void RefreshFilter() = 0;

	/** Get editable skeleton that this widget is editing */
	virtual TSharedRef<class IEditableSkeleton> GetEditableSkeleton() const = 0;

	/** Get preview scene that this widget is editing */
	virtual TSharedPtr<class IPersonaPreviewScene> GetPreviewScene() const = 0;

	/** Set the skeletal mesh we optionally work with */
	virtual void SetSkeletalMesh(class USkeletalMesh* NewSkeletalMesh) = 0;

	/** Set the selected socket */
	virtual void SetSelectedSocket(const struct FSelectedSocketInfo& InSocketInfo) = 0;

	/** Set the selected bone */
	UE_DEPRECATED(4.26, "Please call/implement SetSelectedBone with ESelectInfo")
	virtual void SetSelectedBone(const FName& InBoneName) final { SetSelectedBone(InBoneName, ESelectInfo::Direct); }

	/** Set the selected bone */
	virtual void SetSelectedBone(const FName& InBoneName, ESelectInfo::Type InSelectInfo) = 0;
	
	/** Set the selected bone(s) */
	virtual void SetSelectedBones(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo) = 0;

	/** Deselect everything that is currently selected */
	virtual void DeselectAll() = 0;

	/** Get the selected items */
	virtual TArray<TSharedPtr<ISkeletonTreeItem>> GetSelectedItems() const = 0;

	/** Select items using the passed in predicate  */
	virtual void SelectItemsBy(TFunctionRef<bool(const TSharedRef<ISkeletonTreeItem>& /* InItem */, bool& /* bInOutExpand */)> Predicate, const ESelectInfo::Type SelectInfo = ESelectInfo::Direct) const = 0;

	/** Duplicate the socket and select it */
	virtual void DuplicateAndSelectSocket(const FSelectedSocketInfo& SocketInfoToDuplicate, const FName& NewParentBoneName = FName()) = 0;

	/** Registers a delegate to be called when the selected items have changed */
	virtual FDelegateHandle RegisterOnSelectionChanged(const FOnSkeletonTreeSelectionChanged& Delegate) = 0;

	/** Unregisters a delegate to be called when the selected items have changed */
	virtual void UnregisterOnSelectionChanged(FDelegateHandle DelegateHandle) = 0;

	/** Gets the currently selected blend profile */
	virtual UBlendProfile* GetSelectedBlendProfile() = 0;

	/** Attached the supplied assets to the tree to the specified attach item (bone/socket) */
	virtual void AttachAssets(const TSharedRef<class ISkeletonTreeItem>& TargetItem, const TArray<FAssetData>& AssetData) = 0;

	/** Get the search box widget, if any, for this tree */
	virtual TSharedPtr<SWidget> GetSearchWidget() const = 0;

	/** Get the pinned commands widget, if any, for this tree */
	virtual TSharedPtr<IPinnedCommandList> GetPinnedCommandList() const = 0;

	/** Function to reset the transforms of selected bones */
	virtual void ResetBoneTransforms(const bool bSelectedOnly) = 0;
};
