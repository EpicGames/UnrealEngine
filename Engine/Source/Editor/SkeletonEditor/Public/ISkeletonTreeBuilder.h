// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Misc/TextFilterExpressionEvaluator.h"

#define UE_API SKELETONEDITOR_API

class ISkeletonTreeItem;
enum class ESkeletonTreeFilterResult : int32;
class FTextFilterExpressionEvaluator;

/** Output struct for builders to use */
struct FSkeletonTreeBuilderOutput
{
	FSkeletonTreeBuilderOutput(TArray<TSharedPtr<class ISkeletonTreeItem>>& InItems, TArray<TSharedPtr<class ISkeletonTreeItem>>& InLinearItems)
		: Items(InItems)
		, LinearItems(InLinearItems)
	{}

	/** 
	 * Add an item to the output
	 * @param	InItem			The item to add
	 * @param	InParentName	The name of the item's parent
	 * @param	InParentTypes	The types of items to search. If this is empty all items will be searched.
	 * @param	bAddToHead		Whether to add the item to the start or end of the parent's children array
	 */
	UE_API void Add(const TSharedPtr<class ISkeletonTreeItem>& InItem, const FName& InParentName, TArrayView<const FName> InParentTypes, bool bAddToHead = false);

	/** 
	 * Add an item to the output
	 * @param	InItem			The item to add
	 * @param	InParentName	The name of the item's parent
	 * @param	InParentTypes	The types of items to search. If this is empty all items will be searched.
	 * @param	bAddToHead		Whether to add the item to the start or end of the parent's children array
	 */
	FORCEINLINE void Add(const TSharedPtr<class ISkeletonTreeItem>& InItem, const FName& InParentName, std::initializer_list<FName> InParentTypes, bool bAddToHead = false)
	{
		Add(InItem, InParentName, MakeArrayView(InParentTypes), bAddToHead);
	}

	/** 
	 * Add an item to the output
	 * @param	InItem			The item to add
	 * @param	InParentName	The name of the item's parent
	 * @param	InParentType	The type of items to search. If this is NAME_None all items will be searched.
	 * @param	bAddToHead		Whether to add the item to the start or end of the parent's children array
	 */
	UE_API void Add(const TSharedPtr<class ISkeletonTreeItem>& InItem, const FName& InParentName, const FName& InParentType, bool bAddToHead = false);

	/** 
	 * Find the item with the specified name
	 * @param	InName	The item's name
	 * @param	InTypes	The types of items to search. If this is empty all items will be searched.
	 * @return the item found, or an invalid ptr if it was not found.
	 */
	UE_API TSharedPtr<class ISkeletonTreeItem> Find(const FName& InName, TArrayView<const FName> InTypes) const;

	/** 
	 * Find the item with the specified name
	 * @param	InName	The item's name
	 * @param	InTypes	The types of items to search. If this is empty all items will be searched.
	 * @return the item found, or an invalid ptr if it was not found.
	 */
	FORCEINLINE TSharedPtr<class ISkeletonTreeItem> Find(const FName& InName, std::initializer_list<FName> InTypes) const
	{
		return Find(InName, MakeArrayView(InTypes));
	}

	/** 
	 * Find the item with the specified name
	 * @param	InName	The item's name
	 * @param	InType	The type of items to search. If this is NAME_None all items will be searched.
	 * @return the item found, or an invalid ptr if it was not found.
	 */
	UE_API TSharedPtr<class ISkeletonTreeItem> Find(const FName& InName, const FName& InType) const;

private:
	/** The items that are built by this builder */
	TArray<TSharedPtr<class ISkeletonTreeItem>>& Items;

	/** A linearized list of all items in OutItems (for easier searching) */
	TArray<TSharedPtr<class ISkeletonTreeItem>>& LinearItems;
};

/** Basic filter used when re-filtering the tree */
struct FSkeletonTreeFilterArgs
{
	FSkeletonTreeFilterArgs(TSharedPtr<FTextFilterExpressionEvaluator> InTextFilter)
		: TextFilter(InTextFilter)
		, bFlattenHierarchyOnFilter(true)
	{}

	/** The text filter we are using, if any */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	/** Whether to flatten the hierarchy so filtered items appear in a linear list */
	bool bFlattenHierarchyOnFilter;
};

/** Delegate used to filter an item. */
DECLARE_DELEGATE_RetVal_TwoParams(ESkeletonTreeFilterResult, FOnFilterSkeletonTreeItem, const FSkeletonTreeFilterArgs& /*InArgs*/, const TSharedPtr<class ISkeletonTreeItem>& /*InItem*/);

/** Interface to implement to provide custom build logic to skeleton trees */
class ISkeletonTreeBuilder : public TSharedFromThis<ISkeletonTreeBuilder>
{
public:

	virtual ~ISkeletonTreeBuilder() = default;

	/** Setup this builder with links to the tree and preview scene */
	virtual void Initialize(const TSharedRef<class ISkeletonTree>& InSkeletonTree, const TSharedPtr<class IPersonaPreviewScene>& InPreviewScene, FOnFilterSkeletonTreeItem InOnFilterSkeletonTreeItem) = 0;

	/**
	 * Build an array of skeleton tree items to display in the tree.
	 * @param	Output			The items that are built by this builder
	 */
	virtual void Build(FSkeletonTreeBuilderOutput& Output) = 0;

	/** Apply filtering to the tree items */
	virtual void Filter(const FSkeletonTreeFilterArgs& InArgs, const TArray<TSharedPtr<class ISkeletonTreeItem>>& InItems, TArray<TSharedPtr<class ISkeletonTreeItem>>& OutFilteredItems) = 0;

	/** Allows the builder to contribute to filtering an item */
	virtual ESkeletonTreeFilterResult FilterItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem) = 0;

	/** Get whether this builder is showing bones */
	virtual bool IsShowingBones() const = 0;

	/** Get whether this builder is showing bones */
	virtual bool IsShowingSockets() const = 0;

	/** Get whether this builder is showing bones */
	virtual bool IsShowingAttachedAssets() const = 0;
};

#undef UE_API
