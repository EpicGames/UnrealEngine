// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

/** 
 * Order is important here! 
 * This enum is used internally to the filtering logic and represents an ordering of most filtered (hidden) to 
 * least filtered (highlighted).
 */
enum class ESkeletonTreeFilterResult : int32
{
	/** Hide the item */
	Hidden,

	/** Show the item because child items were shown */
	ShownDescendant,

	/** Show the item */
	Shown,

	/** Show the item and highlight search text */
	ShownHighlighted,
};

#define SKELETON_TREE_BASE_ITEM_TYPE(TYPE) \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	virtual bool IsOfTypeByName(const FName& Type) const { return TYPE::GetTypeId() == Type; } \
	virtual FName GetTypeName() const { return TYPE::GetTypeId(); }

/** Interface for a skeleton tree item */
class ISkeletonTreeItem : public TSharedFromThis<ISkeletonTreeItem>
{
public:
	SKELETON_TREE_BASE_ITEM_TYPE(ISkeletonTreeItem)

	/** Check if this item can cast safely to the specified template type */
	template<typename TType> bool IsOfType() const
	{
		return IsOfTypeByName(TType::GetTypeId());
	}

	virtual ~ISkeletonTreeItem() = default;

	/** Builds the table row widget to display this info */
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, const TAttribute<FText>& InFilterText) = 0;

	/** Builds the slate widget for the name column */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) = 0;

	/** Builds the slate widget for the data column */
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) = 0;

	/** Builds the slate widget for any inline data editing */
	virtual TSharedRef< SWidget > GenerateInlineEditWidget(const TAttribute<FText>& FilterText, FIsSelected InIsSelected) = 0;

	/** @return true if the item has an in-line editor widget */
	virtual bool HasInlineEditor() const = 0;

	/** Toggle the expansion state of the inline editor */
	virtual void ToggleInlineEditorExpansion() = 0;

	/** Get the expansion state of the inline editor */
	virtual bool IsInlineEditorExpanded() const = 0;

	/** Get the name of the item that this row represents */
	virtual FName GetRowItemName() const = 0;

	/** Return the name used to attach to this item */
	virtual FName GetAttachName() const = 0;

	/** @return true if this item can be renamed */
	virtual bool CanRenameItem() const = 0;

	/** Requests a rename on the the tree row item */
	virtual void RequestRename() = 0;

	/** Handler for when the user double clicks on this item in the tree */
	virtual void OnItemDoubleClicked() = 0;

	/** Handle a drag and drop enter event */
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) = 0;

	/** Handle a drag and drop leave event */
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) = 0;

	/** Handle a drag and drop drop event */
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) = 0;

	/** Get this item's parent  */
	virtual TSharedPtr<ISkeletonTreeItem> GetParent() const = 0;

	/** Set this item's parent */
	virtual void SetParent(TSharedPtr<ISkeletonTreeItem> InParent) = 0;

	/** The array of children for this item */
	virtual TArray<TSharedPtr<ISkeletonTreeItem>>& GetChildren() = 0;

	/** The filtered array of children for this item */
	virtual TArray<TSharedPtr<ISkeletonTreeItem>>& GetFilteredChildren() = 0;

	/** The owning skeleton tree */
	virtual TSharedRef<class ISkeletonTree> GetSkeletonTree() const  = 0;

	/** Get the editable skeleton the tree represents */
	virtual TSharedRef<class IEditableSkeleton> GetEditableSkeleton() const = 0;

	/** Get the current filter result */
	virtual ESkeletonTreeFilterResult GetFilterResult() const = 0;

	/** Set the current filter result */
	virtual void SetFilterResult(ESkeletonTreeFilterResult InResult) = 0;

	/** Get the object represented by this item, if any */
	virtual UObject* GetObject() const = 0;

	/** Get whether this item begins expanded or not */
	virtual bool IsInitiallyExpanded() const = 0;
};

/**
 * All concrete ISkeletonTreeItem-derived classes must include this macro.
 * Example Usage:
 *	class FMyTreeItem : public ISkeletonTreeItem
 *	{
 *	public:
 *		SKELETON_TREE_ITEM_TYPE(FMyTreeItem, ISkeletonTreeItem)
 *		...
 *	};
 */
#define SKELETON_TREE_ITEM_TYPE(TYPE, BASE) \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	virtual bool IsOfTypeByName(const FName& Type) const override { return GetTypeId() == Type || BASE::IsOfTypeByName(Type); } \
	virtual FName GetTypeName() const { return TYPE::GetTypeId(); }
