// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISkeletonTreeBuilder.h"
#include "IEditableSkeleton.h"
#include "ISkeletonTreeItem.h"

#define UE_API SKELETONEDITOR_API

class IPersonaPreviewScene;
class ISkeletonTree;
class USkeletalMeshSocket;

/** Options for skeleton building */
struct FSkeletonTreeBuilderArgs
{
	FSkeletonTreeBuilderArgs(bool bInShowBones = true, bool bInShowSockets = true, bool bInShowAttachedAssets = true, bool bInShowVirtualBones = true)
		: bShowBones(bInShowBones)
		, bShowSockets(bInShowSockets)
		, bShowAttachedAssets(bInShowAttachedAssets)
		, bShowVirtualBones(bInShowVirtualBones)
	{}

	/** Whether we should show bones */
	bool bShowBones;

	/** Whether we should show sockets */
	bool bShowSockets;

	/** Whether we should show attached assets */
	bool bShowAttachedAssets;

	/** Whether we should show virtual bones */
	bool bShowVirtualBones;
};

/** Enum which tells us what type of bones we should be showing */
enum class EBoneFilter : int32
{
	All,
	Mesh,
	LOD,
	Weighted, /** only showing weighted bones of current LOD */
	None,
	Count
};

/** Enum which tells us what type of sockets we should be showing */
enum class ESocketFilter : int32
{
	Active,
	Mesh,
	Skeleton,
	All,
	None,
	Count
};

class FSkeletonTreeBuilder : public ISkeletonTreeBuilder
{
public:
	UE_API FSkeletonTreeBuilder(const FSkeletonTreeBuilderArgs& InBuilderArgs);

	/** ISkeletonTreeBuilder interface */
	UE_API virtual void Initialize(const TSharedRef<class ISkeletonTree>& InSkeletonTree, const TSharedPtr<class IPersonaPreviewScene>& InPreviewScene, FOnFilterSkeletonTreeItem InOnFilterSkeletonTreeItem) override;
	UE_API virtual void Build(FSkeletonTreeBuilderOutput& Output) override;
	UE_API virtual void Filter(const FSkeletonTreeFilterArgs& InArgs, const TArray<TSharedPtr<class ISkeletonTreeItem>>& InItems, TArray<TSharedPtr<class ISkeletonTreeItem>>& OutFilteredItems) override;
	UE_API virtual ESkeletonTreeFilterResult FilterItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem) override;
	UE_API virtual bool IsShowingBones() const override;
	UE_API virtual bool IsShowingSockets() const override;
	UE_API virtual bool IsShowingAttachedAssets() const override;

protected:
	/** Add Bones */
	UE_API void AddBones(FSkeletonTreeBuilderOutput& Output);

	/** Add Sockets */
	UE_API void AddSockets(FSkeletonTreeBuilderOutput& Output);

	UE_API void AddAttachedAssets(FSkeletonTreeBuilderOutput& Output);

	UE_API void AddSocketsFromData(const TArray< USkeletalMeshSocket* >& SocketArray, ESocketParentType ParentType, FSkeletonTreeBuilderOutput& Output);

	UE_API void AddAttachedAssetContainer(const FPreviewAssetAttachContainer& AttachedObjects, FSkeletonTreeBuilderOutput& Output);

	UE_API void AddVirtualBones(FSkeletonTreeBuilderOutput& Output);

	/** Create an item for a bone */
	UE_API TSharedRef<class ISkeletonTreeItem> CreateBoneTreeItem(const FName& InBoneName);

	/** Create an item for a socket */
	UE_API TSharedRef<class ISkeletonTreeItem> CreateSocketTreeItem(USkeletalMeshSocket* InSocket, ESocketParentType ParentType, bool bIsCustomized);

	/** Create an item for an attached asset */
	UE_API TSharedRef<class ISkeletonTreeItem> CreateAttachedAssetTreeItem(UObject* InAsset, const FName& InAttachedTo);

	/** Create an item for a virtual bone */
	UE_API TSharedRef<class ISkeletonTreeItem> CreateVirtualBoneTreeItem(const FName& InBoneName);

	/** Helper function for filtering */
	UE_API ESkeletonTreeFilterResult FilterRecursive(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<ISkeletonTreeItem>& InItem, TArray<TSharedPtr<ISkeletonTreeItem>>& OutFilteredItems);

protected:
	/** Options for building */
	FSkeletonTreeBuilderArgs BuilderArgs;

	/** Delegate used for filtering */
	FOnFilterSkeletonTreeItem OnFilterSkeletonTreeItem;

	/** The skeleton tree we will build against */
	TWeakPtr<class ISkeletonTree> SkeletonTreePtr;

	/** The editable skeleton that the skeleton tree represents */
	TWeakPtr<class IEditableSkeleton> EditableSkeletonPtr;

	/** The (optional) preview scene we will build against */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;
};

#undef UE_API
