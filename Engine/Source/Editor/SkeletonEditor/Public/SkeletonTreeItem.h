// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API SKELETONEDITOR_API

class FDragDropEvent;
class FText;
class ITableRow;
class STableViewBase;
class SWidget;
class UObject;
struct FGeometry;
struct FPointerEvent;

/** Provides basic stub functionality for ISkeletonTreeItem-derived classes */
class FSkeletonTreeItem : public ISkeletonTreeItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreeItem, ISkeletonTreeItem)

	friend class FSkeletonTreeBuilder;

	FSkeletonTreeItem(const TSharedRef<class ISkeletonTree>& InSkeletonTree)
		: SkeletonTreePtr(InSkeletonTree)
		, FilterResult(ESkeletonTreeFilterResult::Shown)
	{}

	/** ISkeletonTreeItem interface */
	UE_API virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, const TAttribute<FText>& InFilterText) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override { return SNullWidget::NullWidget; }
	virtual TSharedRef< SWidget > GenerateInlineEditWidget(const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override { return SNullWidget::NullWidget; }
	virtual bool HasInlineEditor() const override { return false; }
	virtual void ToggleInlineEditorExpansion() override {}
	virtual bool IsInlineEditorExpanded() const override { return false; }
	virtual FName GetAttachName() const override { return GetRowItemName(); }
	virtual bool CanRenameItem() const override { return false; }
	virtual void RequestRename() override {}
	virtual void OnItemDoubleClicked() override {}
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) override {}
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override {}
	virtual FReply HandleDrop(const FDragDropEvent& DragDropEvent) override { return FReply::Unhandled(); }
	virtual TSharedPtr<ISkeletonTreeItem> GetParent() const override { return Parent.Pin(); }
	virtual void SetParent(TSharedPtr<ISkeletonTreeItem> InParent) override { Parent = InParent; }
	virtual TArray<TSharedPtr<ISkeletonTreeItem>>& GetChildren() override { return Children; }
	virtual TArray<TSharedPtr<ISkeletonTreeItem>>& GetFilteredChildren() override { return FilteredChildren; }
	virtual TSharedRef<class ISkeletonTree> GetSkeletonTree() const override { return SkeletonTreePtr.Pin().ToSharedRef(); }
	virtual TSharedRef<class IEditableSkeleton> GetEditableSkeleton() const override { return GetSkeletonTree()->GetEditableSkeleton(); }
	virtual ESkeletonTreeFilterResult GetFilterResult() const override { return FilterResult; }
	virtual void SetFilterResult(ESkeletonTreeFilterResult InResult) override { FilterResult = InResult; }
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual UObject* GetObject() const override { return nullptr; }
	virtual bool IsInitiallyExpanded() const override { return true; }

protected:
	/** The parent of this item */
	TWeakPtr<ISkeletonTreeItem> Parent;

	/** The children of this item */
	TArray<TSharedPtr<ISkeletonTreeItem>> Children;

	/** The filtered children of this item */
	TArray<TSharedPtr<ISkeletonTreeItem>> FilteredChildren;

	/** The owning skeleton tree */
	TWeakPtr<class ISkeletonTree> SkeletonTreePtr;

	/** The current filter result */
	ESkeletonTreeFilterResult FilterResult;
};

#undef UE_API
