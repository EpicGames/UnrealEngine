// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/AnyOf.h"
#include "ISkeletonTreeItem.h"

/** Helper for processing tree selections */
class FSkeletonTreeSelection
{
public:
	/** Flat array of selected items */
	TArrayView<TSharedPtr<ISkeletonTreeItem>> SelectedItems;

	FSkeletonTreeSelection(TArray<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems)
		: SelectedItems(InSelectedItems)
	{
	}

	FSkeletonTreeSelection(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems)
		: SelectedItems(InSelectedItems)
	{
	}

	bool IsMultipleItemsSelected() const
	{
		return SelectedItems.Num() > 1;
	}

	bool IsSingleItemSelected() const
	{
		return SelectedItems.Num() == 1;
	}

	template<typename ItemType>
	bool IsSingleOfTypeSelected() const
	{
		if (IsSingleItemSelected())
		{
			return SelectedItems[0]->IsOfType<ItemType>();
		}
		return false;
	}

	template<typename... ItemTypes>
	bool IsSingleOfTypesSelected() const
	{
		if (IsSingleItemSelected())
		{
			const bool bTypeResults[] = { false, (SelectedItems[0]->IsOfType<ItemTypes>())... };
			return Algo::AnyOf(bTypeResults);
		}
		return false;
	}

	TSharedPtr<ISkeletonTreeItem> GetSingleSelectedItem()
	{
		check(IsSingleItemSelected());
		return SelectedItems[0];
	}

	template<typename ItemType>
	bool HasSelectedOfType() const
	{
		for (const TSharedPtr<ISkeletonTreeItem>& SelectedItem : SelectedItems)
		{
			if (SelectedItem->IsOfType<ItemType>())
			{
				return true;
			}
		}

		return false;
	}

	template<typename... ItemTypes>
	bool HasSelectedOfTypes() const
	{
		for (const TSharedPtr<ISkeletonTreeItem>& SelectedItem : SelectedItems)
		{
			const bool bTypeResults[] = { false, (SelectedItem->IsOfType<ItemTypes>())... };
			if (Algo::AnyOf(bTypeResults))
			{
				return true;
			}
		}

		return false;
	}

	template<typename ItemType>
	TArray<TSharedPtr<ItemType>> GetSelectedItems() const
	{
		TArray<TSharedPtr<ItemType>> Items;
		for (const TSharedPtr<ISkeletonTreeItem>& SelectedItem : SelectedItems)
		{
			if (SelectedItem->IsOfType<ItemType>())
			{
				Items.Add(StaticCastSharedPtr<ItemType>(SelectedItem));
			}
		}
		return Items;
	}

	template<typename... ItemTypes>
	TArray<TSharedPtr<ISkeletonTreeItem>> GetSelectedItemsOfTypes() const
	{
		TArray<TSharedPtr<ISkeletonTreeItem>> Items;
		for (const TSharedPtr<ISkeletonTreeItem>& SelectedItem : SelectedItems)
		{
			const bool bTypeResults[] = { false, (SelectedItem->IsOfType<ItemTypes>())... };
			if (Algo::AnyOf(bTypeResults))
			{
				Items.Add(SelectedItem);
			}
		}
		return Items;
	}

	TArray<TSharedPtr<ISkeletonTreeItem>> GetSelectedItemsByTypeId(const FName& InTypeId) const
	{
		TArray<TSharedPtr<ISkeletonTreeItem>> Items;
		for (const TSharedPtr<ISkeletonTreeItem>& SelectedItem : SelectedItems)
		{
			if (SelectedItem->IsOfTypeByName(InTypeId))
			{
				Items.Add(SelectedItem);
			}
		}
		return Items;
	}
};