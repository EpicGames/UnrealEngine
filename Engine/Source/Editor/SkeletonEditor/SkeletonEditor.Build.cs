// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SkeletonEditor : ModuleRules
{
	public SkeletonEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"ApplicationCore",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"Persona",
				"AnimGraph",
				"AnimGraphRuntime",
				"ContentBrowser",
				"AssetRegistry",
				"BlueprintGraph",
				"Kismet",
				"PinnedCommandList",
				"ToolMenus",
				"EditorWidgets",
				"ToolWidgets",
				"AnimationCore",
				"AnimationWidgets", 
			}
		);

        PublicIncludePathModuleNames.AddRange(
            new string[] {
                "Persona",
                "AnimationWidgets",
            }
        );

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "PropertyEditor",
                "MeshBoneReduction",
            }
        );

        DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"MeshBoneReduction",
			}
		);
	}
}
